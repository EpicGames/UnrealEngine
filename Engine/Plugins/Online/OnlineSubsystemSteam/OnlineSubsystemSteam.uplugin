{
	"FileVersion" : 3,
	
	"FriendlyName" : "Online Subsystem Steam",
	"Version" : 1.2,
	"VersionName" : "1.2",
	"Description" : "Access to Steam platform",
	"Category" : "Online Platform",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"EnabledByDefault" : false,

    "Modules": [
        {
            "Name": "OnlineSubsystemSteam",
            "Type": "Runtime",
            "LoadingPhase": "Default",
            "PlatformAllowList" :
			[
                "Win64",
                "Mac",
                "Linux",
				"Android"
            ],
			"PlatformArchitectureDenyList" :
			[
				"Win64:arm64"
			]
        }
    ],
	"LocalizationTargets": [
		{
			"Name": "OnlineSubsystemSteam",
			"LoadingPolicy": "Always"
		}
	],
	"Plugins": [
		{
			"Name": "OnlineSubsystem",
			"Enabled": true
		},
		{
			"Name": "OnlineSubsystemUtils",
			"Enabled": true
		},
		{
			"Name": "SteamShared",
			"Enabled": true
		}
	]

}