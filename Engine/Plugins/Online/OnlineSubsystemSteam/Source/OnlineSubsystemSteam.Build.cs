// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class OnlineSubsystemSteam : ModuleRules
{
	public OnlineSubsystemSteam(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDefinitions.Add("ONLINESUBSYSTEMSTEAM_PACKAGE=1");
		PrivateDefinitions.Add("STEAMSHARED_PACKAGE=1");

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"NetCore",
				"OnlineBase",
				"OnlineSubsystem",
				"Json",
				"Projects",
				"SteamShared",
				"PacketHandler",
				"Sockets",
			}
		);

		if (Target.bCompileAgainstEngine)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
				"Engine",
				"Voice",
				"AudioMixer"
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[] {
				"OnlineSubsystemUtils"
				}
			);

		}

		AddEngineThirdPartyPrivateStaticDependencies(Target, "Steamworks");
	}
}
