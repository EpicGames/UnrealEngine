// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemSteamTypes.h"
#include "NboSerializerOSS.h"

/**
 * Serializes data in network byte order form into a buffer
 */
class FNboSerializeToBufferSteam : public FNboSerializeToBufferOSS
{
public:
	/** Default constructor zeros num bytes*/
	FNboSerializeToBufferSteam() :
		FNboSerializeToBufferOSS(512)
	{
	}

	/** Constructor specifying the size to use */
	FNboSerializeToBufferSteam(uint32 Size) :
		FNboSerializeToBufferOSS(Size)
	{
	}

	/**
	 * Adds Steam session info to the buffer
	 */
 	friend inline FNboSerializeToBufferSteam& operator<<(FNboSerializeToBufferSteam& Ar, const FOnlineSessionInfoSteam& SessionInfo)
 	{
		check(SessionInfo.HostAddr.IsValid());
		// Skip SessionType (assigned at creation)
		((FNboSerializeToBuffer&)Ar) << *SessionInfo.HostAddr;
		((FNboSerializeToBuffer&)Ar) << SessionInfo.SessionId->UniqueNetId;
		return Ar;
 	}
};

/**
 * Class used to write data into packets for sending via system link
 */
class FNboSerializeFromBufferSteam : public FNboSerializeFromBufferOSS
{
public:
	/**
	 * Initializes the buffer, size, and zeros the read offset
	 */
	FNboSerializeFromBufferSteam(uint8* Packet,int32 Length) :
		FNboSerializeFromBufferOSS(Packet,Length)
	{
	}

	/**
	 * Reads Steam session info from the buffer
	 */
 	friend inline FNboSerializeFromBufferSteam& operator>>(FNboSerializeFromBufferSteam& Ar, FOnlineSessionInfoSteam& SessionInfo)
 	{
		check(SessionInfo.HostAddr.IsValid());

		// Skip SessionType (assigned at creation)
		Ar >> *SessionInfo.HostAddr;

		uint64 SessionId;
		Ar >> SessionId;
		SessionInfo.SessionId = FUniqueNetIdSteam::Create(SessionId);

		return Ar;
 	}
};
