// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineAchievementsInterfaceSteam.h"
#include "OnlineSubsystemSteam.h"
#include "OnlineLeaderboardInterfaceSteam.h"
#include <steam/isteamuser.h>

FOnlineAchievementsSteam::FOnlineAchievementsSteam( class FOnlineSubsystemSteam* InSubsystem )
	:	SteamSubsystem(InSubsystem)
{
	check(SteamSubsystem);

	// get leaderboard interface as a lot of functionality is shared with it
	StatsInt = SteamSubsystem->GetInternalLeaderboardsInterface();
	check(StatsInt);

	bHaveConfiguredAchievements = ReadAchievementsFromConfig();
}

bool FOnlineAchievementsSteam::ReadAchievementsFromConfig()
{
	if (Achievements.Num() > 0)
	{
		return true;
	}

	FSteamAchievementsConfig Config;
	return Config.ReadAchievements(Achievements);
}

void FOnlineAchievementsSteam::WriteAchievements(const FUniqueNetId& PlayerId, FOnlineAchievementsWriteRef& WriteObject, const FOnAchievementsWrittenDelegate& Delegate)
{
	if (!bHaveConfiguredAchievements)
	{
		// we don't have achievements
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Steam achievements have not been configured in .ini"));

		WriteObject->WriteState = EOnlineAsyncTaskState::Failed;
		Delegate.ExecuteIfBound(PlayerId, false);
		return;
	}

	const FUniqueNetIdSteam& SteamId = FUniqueNetIdSteam::Cast(PlayerId);
	const CSteamID CSID = SteamId;
	// check if this is our player (cannot report for someone else)
	if (SteamUser() == NULL || SteamUser()->GetSteamID() != CSID)
	{
		// we don't have achievements
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Cannot report Steam achievements for non-local player %s"), *PlayerId.ToString());

		WriteObject->WriteState = EOnlineAsyncTaskState::Failed;
		Delegate.ExecuteIfBound(PlayerId, false);
		return;
	}

	const TArray<FOnlineAchievement> * PlayerAch = PlayerAchievements.Find(SteamId.AsShared());
	if (NULL == PlayerAch)
	{
		// achievements haven't been read for a player
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Steam achievements have not been read for player %s"), *PlayerId.ToString());

		WriteObject->WriteState = EOnlineAsyncTaskState::Failed;
		Delegate.ExecuteIfBound(PlayerId, false);
		return;
	}

	const int32 AchNum = PlayerAch->Num();
	for (FStatPropertyArray::TConstIterator It(WriteObject->Properties); It; ++It)
	{
		const FString AchievementId = It.Key();
		UE_LOG_ONLINE_ACHIEVEMENTS(Verbose, TEXT("WriteObject AchievementId: '%s'"), *AchievementId);
		for (int32 AchIdx = 0; AchIdx < AchNum; ++AchIdx)
		{
			if ((*PlayerAch)[ AchIdx ].Id == AchievementId)
			{
				// do not unlock it now, but after a successful write
#if !UE_BUILD_SHIPPING
				float Value = 0.0f;
				It.Value().GetValue(Value);
				if (Value <= 0.0f)
				{
					UE_LOG_ONLINE_ACHIEVEMENTS(Verbose, TEXT("Resetting achievement '%s'"), *AchievementId);
					SteamUserStats()->ClearAchievement(TCHAR_TO_UTF8(*AchievementId));
				}
				else
				{
#endif // !UE_BUILD_SHIPPING

					UE_LOG_ONLINE_ACHIEVEMENTS(Verbose, TEXT("Setting achievement '%s'"), *AchievementId);
					SteamUserStats()->SetAchievement(TCHAR_TO_UTF8(*AchievementId));

#if !UE_BUILD_SHIPPING
				}
#endif // !UE_BUILD_SHIPPING
			
				break;
			}
		}
	}

	StatsInt->WriteAchievementsInternal(SteamId, WriteObject, Delegate);
};

void FOnlineAchievementsSteam::OnWriteAchievementsComplete(const FUniqueNetIdSteam& PlayerId, bool bWasSuccessful, FOnlineAchievementsWritePtr& WriteObject, const FOnAchievementsWrittenDelegate& Delegate)
{
	// write object should be valid
	check(WriteObject.IsValid());

	// if write completed successfully, unlock the achievements (and update their cache)
	if (bWasSuccessful)
	{
		TArray<FOnlineAchievement> * PlayerAch = PlayerAchievements.Find(PlayerId.AsShared());
		check(PlayerAch);	// were we writing for a non-existing player?
		if (NULL != PlayerAch && WriteObject.IsValid())
		{
			// treat each achievement as unlocked
			const int32 AchNum = PlayerAch->Num();
			for (FStatPropertyArray::TConstIterator It(WriteObject->Properties); It; ++It)
			{
				const FString AchievementId = It.Key();
				for (int32 AchIdx = 0; AchIdx < AchNum; ++AchIdx)
				{
					if ((*PlayerAch)[ AchIdx ].Id == AchievementId)
					{
						// Update and trigger the callback
						(*PlayerAch)[ AchIdx ].Progress = 100.0;
						TriggerOnAchievementUnlockedDelegates(PlayerId, (*PlayerAch)[ AchIdx ].Id);
						break;
					}
				}
			}
		}
	}

	Delegate.ExecuteIfBound(PlayerId, bWasSuccessful);
}

void FOnlineAchievementsSteam::QueryAchievements(const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate)
{
	if (!bHaveConfiguredAchievements)
	{
		// we don't have achievements
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Steam achievements have not been configured in .ini"));

		Delegate.ExecuteIfBound( PlayerId, false );
		return;
	}

	// schedule a read (this will trigger the OnAchievementsRead delegates)
	StatsInt->QueryAchievementsInternal( FUniqueNetIdSteam::Cast(PlayerId), Delegate );
}

void FOnlineAchievementsSteam::QueryAchievementDescriptions( const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate )
{
	if (!bHaveConfiguredAchievements)
	{
		// we don't have achievements
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Steam achievements have not been configured in .ini"));

		Delegate.ExecuteIfBound( PlayerId, false );
		return;
	}

	// schedule a read (this will trigger the OnAchievementDescriptionsRead delegates)
	StatsInt->QueryAchievementsInternal( FUniqueNetIdSteam::Cast(PlayerId), Delegate );
}


/**
 * ** INTERNAL **
 * Called by an async task after completing an achievement read.
 *
 * @param PlayerId - id of a player we were making read for
 * @param bReadSuccessfully - whether the read completed successfully
 */
void FOnlineAchievementsSteam::UpdateAchievementsForUser(const FUniqueNetIdSteam& PlayerId, bool bReadSuccessfully)
{
	// shouldn't get this far if no achievements are configured
	check(bHaveConfiguredAchievements);

	ISteamUserStats* SteamUserStatsPtr = SteamUserStats();
	check(SteamUserStatsPtr);
	CSteamID SteamUserId = PlayerId;

	// new array
	TArray<FOnlineAchievement> AchievementsForPlayer;
	const int32 AchNum = Achievements.Num();

	for (int32 AchIdx = 0; AchIdx < AchNum; ++AchIdx)
	{
		// get the info
		bool bUnlocked;
		uint32 UnlockUnixTime;
		if (!SteamUserStatsPtr->GetAchievementAndUnlockTime(TCHAR_TO_UTF8(*Achievements[AchIdx].Id), &bUnlocked, &UnlockUnixTime))
		{
			UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("GetAchievementAndUnlockTime() failed for achievement '%s'"), *Achievements[AchIdx].Id);
			// skip this achievement
			continue;
		}

		FOnlineAchievementSteam NewAch = Achievements[ AchIdx ];
		NewAch.bReadFromSteam = true;
		NewAch.Progress = bUnlocked ? 100.0 : 0.0;	// TODO: we may want to support more fine-grained progress based on associated stat and min/max, 
													// although we can only map it one-way (i.e. when reading, but not when writing)
		NewAch.UnlockTime = FDateTime::FromUnixTimestamp(UnlockUnixTime);

		NewAch.Title = FText::FromString( UTF8_TO_TCHAR( SteamUserStatsPtr->GetAchievementDisplayAttribute(TCHAR_TO_UTF8(*Achievements[AchIdx].Id), "name") ) );
		NewAch.LockedDesc = FText::FromString( UTF8_TO_TCHAR( SteamUserStatsPtr->GetAchievementDisplayAttribute(TCHAR_TO_UTF8(*Achievements[AchIdx].Id), "desc") ) );
		NewAch.UnlockedDesc = NewAch.LockedDesc;

		NewAch.bIsHidden = FCString::Atoi( UTF8_TO_TCHAR( SteamUserStatsPtr->GetAchievementDisplayAttribute(TCHAR_TO_UTF8(*Achievements[AchIdx].Id), "hidden") ) ) != 0;

		UE_LOG_ONLINE_ACHIEVEMENTS(Verbose, TEXT("Read achievement %d: %s"), AchIdx, *NewAch.ToDebugString());
		AchievementsForPlayer.Add( NewAch );

		// add mapping (should replace any existing one)
		AchievementDescriptions.Add(NewAch.Id, NewAch);
	}

	// should replace any already existing values
	PlayerAchievements.Add(PlayerId.AsShared(), AchievementsForPlayer);
}

EOnlineCachedResult::Type FOnlineAchievementsSteam::GetCachedAchievement(const FUniqueNetId& PlayerId, const FString& AchievementId, FOnlineAchievement& OutAchievement)
{
	if (!bHaveConfiguredAchievements)
	{
		// we don't have achievements
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Steam achievements have not been configured in .ini"));
		return EOnlineCachedResult::NotFound;
	}

	const TArray<FOnlineAchievement> * PlayerAch = PlayerAchievements.Find(PlayerId.AsShared());
	if (NULL == PlayerAch)
	{
		// achievements haven't been read for a player
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Steam achievements have not been read for player %s"), *PlayerId.ToString());
		return EOnlineCachedResult::NotFound;
	}

	const int32 AchNum = PlayerAch->Num();
	for (int32 AchIdx = 0; AchIdx < AchNum; ++AchIdx)
	{
		if ((*PlayerAch)[ AchIdx ].Id == AchievementId)
		{
			OutAchievement = (*PlayerAch)[ AchIdx ];
			return EOnlineCachedResult::Success;
		}
	}

	// no such achievement
	UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Could not find Steam achievement '%s' for player %s"), *AchievementId, *PlayerId.ToString());
	return EOnlineCachedResult::NotFound;
};

EOnlineCachedResult::Type FOnlineAchievementsSteam::GetCachedAchievements(const FUniqueNetId& PlayerId, TArray<FOnlineAchievement> & OutAchievements)
{
	if (!bHaveConfiguredAchievements)
	{
		// we don't have achievements
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Steam achievements have not been configured in .ini"));
		return EOnlineCachedResult::NotFound;
	}

	const TArray<FOnlineAchievement> * PlayerAch = PlayerAchievements.Find(PlayerId.AsShared());
	if (NULL == PlayerAch)
	{
		// achievements haven't been read for a player
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Steam achievements have not been read for player %s"), *PlayerId.ToString());
		return EOnlineCachedResult::NotFound;
	}

	OutAchievements = *PlayerAch;
	return EOnlineCachedResult::Success;
};

EOnlineCachedResult::Type FOnlineAchievementsSteam::GetCachedAchievementDescription(const FString& AchievementId, FOnlineAchievementDesc& OutAchievementDesc)
{
	if (!bHaveConfiguredAchievements)
	{
		// we don't have achievements
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Steam achievements have not been configured in .ini"));
		return EOnlineCachedResult::NotFound;
	}

	if (AchievementDescriptions.Num() == 0 )
	{
		// don't have descs
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Descriptions have not been read"));
		return EOnlineCachedResult::NotFound;
	}

	FOnlineAchievementDesc * AchDesc = AchievementDescriptions.Find(AchievementId);
	if (NULL == AchDesc)
	{
		// no such achievement
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Achievement '%s' does not have a description"), *AchievementId);
		return EOnlineCachedResult::NotFound;
	}

	OutAchievementDesc = *AchDesc;
	return EOnlineCachedResult::Success;
};

#if !UE_BUILD_SHIPPING
bool FOnlineAchievementsSteam::ResetAchievements(const FUniqueNetId& PlayerId)
{
	if (!bHaveConfiguredAchievements)
	{
		// we don't have achievements
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Steam achievements have not been configured in .ini"));
		return false;
	}

	const FUniqueNetIdSteam& SteamId = FUniqueNetIdSteam::Cast(PlayerId);
	// check if this is our player (cannot report for someone else)
	if (SteamUser() == NULL || SteamUser()->GetSteamID() != SteamId)
	{
		// we don't have achievements
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Cannot clear Steam achievements for non-local player %s"), *PlayerId.ToString());
		return false;
	}

	const TArray<FOnlineAchievement> * PlayerAch = PlayerAchievements.Find(SteamId.AsShared());
	if (NULL == PlayerAch)
	{
		// achievements haven't been read for a player
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Steam achievements have not been read for player %s"), *PlayerId.ToString());
		return false;
	}

	const int32 AchNum = PlayerAch->Num();
	for (int32 AchIdx = 0; AchIdx < AchNum; ++AchIdx)
	{
		SteamUserStats()->ClearAchievement(TCHAR_TO_UTF8(*(*PlayerAch)[ AchIdx ].Id));
	}

	// TODO: provide a separate method to just store stats?
	StatsInt->FlushLeaderboards(FName(TEXT("UNUSED")));
	return true;
};
#endif // !UE_BUILD_SHIPPING
