// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineAchievementsInterface.h"
#include "Misc/ConfigCacheIni.h"
#include "OnlineSubsystemPackage.h"

class FUniqueNetIdSteam;

/**
 *	IOnlineAchievements - Interface class for acheivements
 */
class FOnlineAchievementsSteam : public IOnlineAchievements
{
private:

	/** Structure describing a Steam achievement */
	struct FOnlineAchievementSteam : public FOnlineAchievement, public FOnlineAchievementDesc
	{
		/** Whether achievement info has been read from Steam */
		bool bReadFromSteam;

		/** Returns debugging string to print out achievement info */
		FString ToDebugString() const
		{
			return FString::Printf( TEXT("Achievement:{%s},  Description:{%s}, bReadFromSteam=%s"),
				*FOnlineAchievement::ToDebugString(),
				*FOnlineAchievementDesc::ToDebugString(),
				bReadFromSteam ? TEXT("true") : TEXT("false")
				);
		}
	};

	/**
	 * A helper class for configuring achievements in ini
	 */
	struct FSteamAchievementsConfig
	{
		/** 
		 * Create a config using the default values:
		 * IniName - GEngineIni
		 */
		FSteamAchievementsConfig()
			:	IniName(GEngineIni)
			,	SectionName(TEXT("OnlineSubsystemSteam"))
		{
		}

		/** Returns empty string if couldn't read */
		FString GetKey(const FString& KeyName)
		{
			FString Result;
			if (!GConfig->GetString(*SectionName, *KeyName, Result, IniName))
			{
				return TEXT("");	// could just return Result, but being explicit is better
			}
			return Result;
		}

		bool ReadAchievements(TArray<FOnlineAchievementSteam> & OutArray)
		{
			OutArray.Empty();
			int NumAchievements = 0;

			for(;;++NumAchievements)
			{
				FString Id = GetKey(FString::Printf(TEXT("Achievement_%d_Id"), NumAchievements));
				if (Id.IsEmpty())
				{
					break;
				}

				FOnlineAchievementSteam NewAch;
				NewAch.Id = Id;
				NewAch.Progress = 0.0;
				NewAch.bReadFromSteam = false;
				
				OutArray.Add(NewAch);
			}

			return NumAchievements > 0;
		}


		/** Ini file name to find the config values */
		FString IniName;
		/** Section name for Steam */
		FString SectionName;
	};

	/** Reference to the owning subsystem */
	class FOnlineSubsystemSteam* SteamSubsystem;

	/** Reference to leaderboard interface (internal class) */
	class FOnlineLeaderboardsSteam* StatsInt;

	/** hide the default constructor, we need a reference to our OSS */
	FOnlineAchievementsSteam() {};

	/** Mapping of players to their achievements */
	TUniqueNetIdMap<TArray<FOnlineAchievement>> PlayerAchievements;

	/** Cached achievement descriptions for an Id */
	TMap<FString, FOnlineAchievementDesc> AchievementDescriptions;

	/** Achievements configured in the config (not player-specific) */
	TArray<FOnlineAchievementSteam> Achievements;

	/** Whether we have achievements specified in the .ini */
	bool bHaveConfiguredAchievements;

	/** Initializes achievements from config. Returns true if there is at least one achievement */
	bool ReadAchievementsFromConfig();

PACKAGE_SCOPE:

	/**
	 * ** INTERNAL **
	 * Called by an async task after completing an achievement read.
	 *
	 * @param PlayerId - id of a player we were making read for
	 * @param bReadSuccessfully - whether the read completed successfully
	 */
	void UpdateAchievementsForUser(const FUniqueNetIdSteam& PlayerId, bool bReadSuccessfully);

	/**
	 * ** INTERNAL **
	 * Called by an async task after completing an achievement write. Used to conditionally unlock achievements in case the write was successful.
	 *
	 * @param PlayerId - id of a player we were making read for
	 * @param bWasSuccessful - whether the read completed successfully
	 * @param WriteObject - achievements we have been writing (unlocking)
	 * @param Delegate - delegate back to the game code indicating the write is completed
	 */
	void OnWriteAchievementsComplete(const FUniqueNetIdSteam& PlayerId, bool bWasSuccessful, FOnlineAchievementsWritePtr & WriteObject, const FOnAchievementsWrittenDelegate& Delegate);

public:

	//~ Begin IOnlineAchievements Interface
	virtual void WriteAchievements(const FUniqueNetId& PlayerId, FOnlineAchievementsWriteRef& WriteObject, const FOnAchievementsWrittenDelegate& Delegate = FOnAchievementsWrittenDelegate()) override;
	virtual void QueryAchievements(const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate = FOnQueryAchievementsCompleteDelegate()) override;
	virtual void QueryAchievementDescriptions( const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate = FOnQueryAchievementsCompleteDelegate() ) override;
	virtual EOnlineCachedResult::Type GetCachedAchievement(const FUniqueNetId& PlayerId, const FString& AchievementId, FOnlineAchievement& OutAchievement) override;
	virtual EOnlineCachedResult::Type GetCachedAchievements(const FUniqueNetId& PlayerId, TArray<FOnlineAchievement> & OutAchievements) override;
	virtual EOnlineCachedResult::Type GetCachedAchievementDescription(const FString& AchievementId, FOnlineAchievementDesc& OutAchievementDesc) override;
#if !UE_BUILD_SHIPPING
	virtual bool ResetAchievements( const FUniqueNetId& PlayerId ) override;
#endif // !UE_BUILD_SHIPPING
	//~ End IOnlineAchievements Interface

	/**
	 * Constructor
	 *
	 * @param InSubsystem - A reference to the owning subsystem
	 */
	FOnlineAchievementsSteam(class FOnlineSubsystemSteam* InSubsystem);

	
	/**
	 * Default destructor
	 */
	virtual ~FOnlineAchievementsSteam(){}
};
