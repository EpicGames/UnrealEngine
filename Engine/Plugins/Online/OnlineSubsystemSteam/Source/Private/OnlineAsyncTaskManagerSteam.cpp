// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineAsyncTaskManagerSteam.h"
#include "OnlinePresenceInterfaceSteam.h"
#include "OnlineSessionAsyncLobbySteam.h"
#include "OnlineSessionAsyncServerSteam.h"
#include "OnlineLeaderboardInterfaceSteam.h"
#include "OnlineExternalUIInterfaceSteam.h"
#include "OnlineAuthInterfaceSteam.h"
#include "SteamUtilities.h"

void FOnlineAsyncTaskManagerSteam::OnlineTick()
{
	check(SteamSubsystem);
	check(FPlatformTLS::GetCurrentThreadId() == OnlineThreadId);

	if (SteamSubsystem->IsSteamClientAvailable())
	{
		SteamAPI_RunCallbacks();
	}

	if (SteamSubsystem->IsSteamServerAvailable())
	{
		SteamGameServer_RunCallbacks();
	}
}

/**
 *	Event triggered by Steam backend when a user attempts JIP or accepts an invite request (via Steam client)
 *
 * @param CallbackData All the valid data from Steam related to this event 
 */
void FOnlineAsyncTaskManagerSteam::OnInviteAccepted(GameRichPresenceJoinRequested_t* CallbackData)
{
	FOnlineAsyncEventSteamInviteAccepted* NewEvent = 
		new FOnlineAsyncEventSteamInviteAccepted(SteamSubsystem, *FUniqueNetIdSteam::Create(CallbackData->m_steamIDFriend), UTF8_TO_TCHAR(CallbackData->m_rgchConnect));
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

/**
 *	Event triggered by Steam backend when a user attempts JIP (via Steam client) or accepts an invite request (via Steam client)
 *
 * @param CallbackData All the valid data from Steam related to this event 
 */
void FOnlineAsyncTaskManagerSteam::OnLobbyInviteAccepted(GameLobbyJoinRequested_t* CallbackData)
{
	if (CallbackData->m_steamIDLobby.IsLobby())
	{
		const FUniqueNetIdSteamRef LobbyId = FUniqueNetIdSteam::Create(CallbackData->m_steamIDLobby);

		FOnlineSessionSteamPtr SessionInt = StaticCastSharedPtr<FOnlineSessionSteam>(SteamSubsystem->GetSessionInterface());
		if (SessionInt.IsValid() && !SessionInt->IsMemberOfLobby(*LobbyId))
		{
			FOnlineAsyncEventSteamLobbyInviteAccepted* NewEvent = 
				new FOnlineAsyncEventSteamLobbyInviteAccepted(SteamSubsystem, *FUniqueNetIdSteam::Create(CallbackData->m_steamIDFriend), *LobbyId);
			UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
			AddToOutQueue(NewEvent);
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("Attempting to accept invite to lobby user is already in, ignoring."));
		}
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("OnLobbyInviteAccepted: Invalid LobbyId received."));
	}
}

/**
 * Notification event from Steam that the lobby state has changed (users joining/leaving)
 */
class FOnlineAsyncEventSteamLobbyEnter : public FOnlineAsyncEvent<FOnlineSubsystemSteam>
{
private:
	
	/** Lobby enter state information */
	LobbyEnter_t CallbackResults;

	/** Hidden on purpose */
	FOnlineAsyncEventSteamLobbyEnter() :
		FOnlineAsyncEvent(NULL)
	{
		FMemory::Memzero(CallbackResults);
	}

public:

	FOnlineAsyncEventSteamLobbyEnter(FOnlineSubsystemSteam* InSubsystem, const LobbyEnter_t& InResults) :
		FOnlineAsyncEvent(InSubsystem),
		CallbackResults(InResults)
	{
	}
	
	virtual ~FOnlineAsyncEventSteamLobbyEnter() 
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncEventSteamLobbyEnter LobbyId: %s Result: %s"), 
			*FUniqueNetIdSteam::ToDebugString(CSteamID(CallbackResults.m_ulSteamIDLobby)),
			*SteamChatRoomEnterResponseString((EChatRoomEnterResponse)CallbackResults.m_EChatRoomEnterResponse));	
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override
	{
		FOnlineSessionSteamPtr SessionInt = StaticCastSharedPtr<FOnlineSessionSteam>(Subsystem->GetSessionInterface());
		if (SessionInt.IsValid())
		{
			const FUniqueNetIdSteamRef LobbyId = FUniqueNetIdSteam::Create(CallbackResults.m_ulSteamIDLobby);
			const FNamedOnlineSession* Session = SessionInt->GetNamedSessionFromLobbyId(*LobbyId);
			if (!Session)
			{
				UE_LOG_ONLINE(Warning, TEXT("Entered lobby %s, but not found in sessions list"), *LobbyId->ToDebugString());
			}
		}
	}
};

/**
 *	Event triggered by Steam backend when a user joins a lobby
 *
 * @param CallbackData All the valid data from Steam related to this event 
 */
void FOnlineAsyncTaskManagerSteam::OnLobbyEnter(LobbyEnter_t* CallbackData)
{
	// The owner of the created lobby shouldn't need this information
	if (SteamMatchmaking()->GetLobbyOwner(CallbackData->m_ulSteamIDLobby) != SteamUser()->GetSteamID())
	{
		FOnlineAsyncEventSteamLobbyEnter* NewEvent = new FOnlineAsyncEventSteamLobbyEnter(SteamSubsystem, *CallbackData);
		UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
		AddToOutQueue(NewEvent);
	}
}

/**
 * Notification event from Steam that the lobby state has changed (users joining/leaving)
 */
class FOnlineAsyncEventSteamLobbyChatUpdate : public FOnlineAsyncEvent<FOnlineSubsystemSteam>
{
private:
	
	/** Lobby chat state information */
	LobbyChatUpdate_t CallbackResults;

	/** Hidden on purpose */
	FOnlineAsyncEventSteamLobbyChatUpdate() :
		FOnlineAsyncEvent(NULL)
	{
		FMemory::Memzero(CallbackResults);
	}

public:

	FOnlineAsyncEventSteamLobbyChatUpdate(FOnlineSubsystemSteam* InSubsystem, const LobbyChatUpdate_t& InResults) :
		FOnlineAsyncEvent(InSubsystem),
		CallbackResults(InResults)
	{
	}
	
	virtual ~FOnlineAsyncEventSteamLobbyChatUpdate() 
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncEventSteamLobbyChatUpdate User: %s Instigator: %s Result: %s"), 
			*FUniqueNetIdSteam::ToDebugString(CSteamID(CallbackResults.m_ulSteamIDUserChanged)),
			*FUniqueNetIdSteam::ToDebugString(CSteamID(CallbackResults.m_ulSteamIDMakingChange)),
			*SteamChatMemberStateChangeString((EChatMemberStateChange)CallbackResults.m_rgfChatMemberStateChange));	
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override
	{
		FOnlineSessionSteamPtr SessionInt = StaticCastSharedPtr<FOnlineSessionSteam>(Subsystem->GetSessionInterface());
		if (SessionInt.IsValid())
		{
			FUniqueNetIdSteamRef LobbyId = FUniqueNetIdSteam::Create(CallbackResults.m_ulSteamIDLobby);
			// Lobby data update for existing session
			FNamedOnlineSession* Session = SessionInt->GetNamedSessionFromLobbyId(*LobbyId);
			if (Session)
			{
				// Recreate the lobby member list
				if (!FillMembersFromLobbyData(*LobbyId, *Session))
				{
					UE_LOG_ONLINE(Warning, TEXT("Failed to parse session %s member update %s"), *Session->SessionName.ToString(), *LobbyId->ToDebugString());
				}
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("Received lobby chat update %s, but not found in sessions list"), *LobbyId->ToDebugString());
			}
		}
	}
};

/**
 *	Event triggered by Steam backend when a user joins a lobby
 *
 * @param CallbackData All the valid data from Steam related to this event 
 */
void FOnlineAsyncTaskManagerSteam::OnLobbyChatUpdate(LobbyChatUpdate_t* CallbackData)
{
	FOnlineAsyncEventSteamLobbyChatUpdate* NewEvent = new FOnlineAsyncEventSteamLobbyChatUpdate(SteamSubsystem, *CallbackData);
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

/**
 * Notification event from Steam when new lobby data is available for the given lobby 
 */
class FOnlineAsyncEventSteamLobbyUpdate : public FOnlineAsyncEvent<FOnlineSubsystemSteam> 
{
private:

	/** Id of lobby to update */
	FUniqueNetIdSteamRef LobbyId;
	/** Hidden on purpose */
	FOnlineAsyncEventSteamLobbyUpdate() = delete;

public:
	FOnlineAsyncEventSteamLobbyUpdate(FOnlineSubsystemSteam* InSubsystem, const FUniqueNetIdSteam& InLobbyId) :
		FOnlineAsyncEvent(InSubsystem),
		LobbyId(InLobbyId.AsShared())
	{
	}

	virtual ~FOnlineAsyncEventSteamLobbyUpdate() 
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncEventSteamLobbyUpdate LobbyId: %s"), *LobbyId->ToDebugString());	
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override
	{
		FOnlineAsyncEvent::Finalize();

		// Searching for lobbies case (NULL CurrentSessionSearch implies no active search query)
		FOnlineSessionSteamPtr SessionInt = StaticCastSharedPtr<FOnlineSessionSteam>(Subsystem->GetSessionInterface());
		if (SessionInt.IsValid() && SessionInt->CurrentSessionSearch.IsValid() && SessionInt->CurrentSessionSearch->SearchState == EOnlineAsyncTaskState::InProgress)
		{
			// Add this lobby as available for adding to search results
			SessionInt->PendingSearchLobbyIds.AddUnique(LobbyId);
		}
		else
		{
			// Lobby data update for existing session
			FNamedOnlineSession* Session = SessionInt->GetNamedSessionFromLobbyId(*LobbyId);
			if (Session)
			{
				// Make sure the session has all the valid session data
				if (!FillSessionFromLobbyData(Subsystem, *LobbyId, *Session) ||
					!FillMembersFromLobbyData(*LobbyId, *Session))
				{
					UE_LOG_ONLINE(Warning, TEXT("Failed to parse session %s lobby update %s"), *Session->SessionName.ToString(), *LobbyId->ToDebugString());
				}
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("Received lobby update %s, but not found in sessions list"), *LobbyId->ToDebugString());
			}
		}
	}
};

/**
 *	Event triggered by Steam backend when new lobby data is available for the given lobby 
 * Can occur any time host calls SetLobbyData or while searching for lobbies (calls to RequestLobbyData)
 *
 * @param CallbackData All the valid data from Steam related to this event 
 */
void FOnlineAsyncTaskManagerSteam::OnLobbyDataUpdate(LobbyDataUpdate_t* CallbackData)
{
	// Equivalent lobby ids implies it is lobby data that has updated
	if (CallbackData->m_ulSteamIDLobby == CallbackData->m_ulSteamIDMember)
	{
		if (!CallbackData->m_bSuccess)
		{
			// CallbackData->m_bSuccess indicates LobbyID has shut down since 
			// the result was returned but we have to keep the array size in sync
			UE_LOG_ONLINE(Verbose, TEXT("Lobby %s is no longer available."), *FUniqueNetIdSteam::ToDebugString(CallbackData->m_ulSteamIDLobby));
		}

		ISteamMatchmaking* SteamMatchmakingPtr = SteamMatchmaking();
		check(SteamMatchmakingPtr);

		// The owner of the created lobby shouldn't need this information
		if (SteamMatchmakingPtr->GetLobbyOwner(CallbackData->m_ulSteamIDLobby) != SteamUser()->GetSteamID())
		{
			FOnlineAsyncEventSteamLobbyUpdate* NewEvent = new FOnlineAsyncEventSteamLobbyUpdate(SteamSubsystem, *FUniqueNetIdSteam::Create(CallbackData->m_ulSteamIDLobby));
			UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
			AddToOutQueue(NewEvent);
		}
	}
	else
	{
		// @TODO ONLINE - Player data update
		//const TCHAR* Key = MemberKeyList(i);
		//const ANSICHAR* MemberValue = SteamMatchmakingPtr->GetLobbyMemberData(CallbackData->m_ulSteamIDLobby, CallbackData->m_ulSteamIDMember, TCHAR_TO_UTF8(Key));
	}

	// @TODO ONLINE - SetLobbyOwner triggers this call also
}

/**
 * Notification event from Steam that a given user's
 * stats/achievements data has been downloaded from the server
 */
class FOnlineAsyncEventSteamStatsReceived : public FOnlineAsyncEvent<FOnlineSubsystemSteam> 
{
private:

	/** User this data is for */
	const FUniqueNetIdSteamRef UserId;
	/** Result of the download */
	EResult StatsReceivedResult;

	/** Hidden on purpose */
	FOnlineAsyncEventSteamStatsReceived() = delete;

public:
	FOnlineAsyncEventSteamStatsReceived(FOnlineSubsystemSteam* InSubsystem, const FUniqueNetIdSteam& InUserId, EResult InResult) :
		FOnlineAsyncEvent(InSubsystem),
		UserId(InUserId.AsShared()),
		StatsReceivedResult(InResult)
	{
	}

	virtual ~FOnlineAsyncEventSteamStatsReceived() 
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncEventSteamStatsReceived bWasSuccessful: %d User: %s Result: %s"), (StatsReceivedResult == k_EResultOK) ? 1 : 0, *UserId->ToDebugString(), *SteamResultString(StatsReceivedResult));	
	}
};

/**
 *	Event triggered from Steam when the current user's stats have been downloaded from the backend
 *	Possible that the result fails if they have no data for the current game
 *
 * @param CallbackData All the valid data from Steam related to this event 
 */
void FOnlineAsyncTaskManagerSteam::OnUserStatsReceived(UserStatsReceived_t* CallbackData)
{
	const CGameID GameID(SteamSubsystem->GetSteamAppId());
	if (GameID.ToUint64() == CallbackData->m_nGameID)
	{
		const FUniqueNetIdSteamRef UserId = FUniqueNetIdSteam::Create(CallbackData->m_steamIDUser);
		if (CallbackData->m_eResult != k_EResultOK)
		{
			if (CallbackData->m_eResult == k_EResultFail)
			{
				UE_LOG_ONLINE(Warning, TEXT("Failed to obtain steam user stats, user: %s has no stats entries"), *UserId->ToDebugString());
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("Failed to obtain steam user stats, user: %s error: %s"), *UserId->ToDebugString(), 
					*SteamResultString(CallbackData->m_eResult));
			}
		}

		FOnlineAsyncEventSteamStatsReceived* NewEvent = new FOnlineAsyncEventSteamStatsReceived(SteamSubsystem, *UserId, CallbackData->m_eResult);
		UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
		AddToOutQueue(NewEvent);
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("Obtained steam user stats, but for wrong game! Ignoring."));
	}
}

/**
 * Notification event from Steam that the currently logged in user's
 * stats/achievements data has been stored with the server
 */
class FOnlineAsyncEventSteamStatsStored : public FOnlineAsyncEvent<FOnlineSubsystemSteam>
{
private:
	
	/** User this data is for */
	const FUniqueNetIdSteamRef UserId;
	/** Result of the download */
	EResult StatsStoredResult;

	/** Hidden on purpose */
	FOnlineAsyncEventSteamStatsStored() = delete;

public:

	FOnlineAsyncEventSteamStatsStored(FOnlineSubsystemSteam* InSubsystem, const FUniqueNetIdSteam& InUserId, EResult InResult) :
		FOnlineAsyncEvent(InSubsystem),
		UserId(InUserId.AsShared()),
		StatsStoredResult(InResult)
	{
	}
	
	virtual ~FOnlineAsyncEventSteamStatsStored() 
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncEventSteamStatsStored bWasSuccessful: %d User: %s Result: %s"), (StatsStoredResult == k_EResultOK) ? 1 : 0, *UserId->ToDebugString(), *SteamResultString(StatsStoredResult));	
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override
	{
		FOnlineAsyncEvent::Finalize();
		FOnlineLeaderboardsSteamPtr Leaderboards = StaticCastSharedPtr<FOnlineLeaderboardsSteam>(Subsystem->GetLeaderboardsInterface());
		Leaderboards->UserStatsStoreStatsFinishedDelegate.ExecuteIfBound(StatsStoredResult == k_EResultOK ? EOnlineAsyncTaskState::Done : EOnlineAsyncTaskState::Failed);
	}
};

/**
 *	Event triggered from Steam when the current user's stats have been stored with the backend
 *	Possible that the result fails with "Invalid Param" meaning the stats went out of range or were out of date
 *  New stats are downloaded in this case and need to be re-evaluated
 *
 * @param CallbackData All the valid data from Steam related to this event 
 */
void FOnlineAsyncTaskManagerSteam::OnUserStatsStored(UserStatsStored_t* CallbackData)
{
	const CGameID GameID(SteamSubsystem->GetSteamAppId());
	if (GameID.ToUint64() == CallbackData->m_nGameID)
	{
		// Only the current user comes through this way (other user's stats are stored via GameServerStats)
		const FUniqueNetIdSteamRef UserId = FUniqueNetIdSteam::Create(SteamUser()->GetSteamID());
		if (CallbackData->m_eResult != k_EResultOK)
		{
			if (CallbackData->m_eResult == k_EResultInvalidParam)
			{
				UE_LOG_ONLINE(Warning, TEXT("Invalid stats data set, stats have been reverted to state prior to last write."));
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("Failed to store steam user stats, error: %s"), *SteamResultString(CallbackData->m_eResult));
			}
		}

		FOnlineAsyncEventSteamStatsStored* NewEvent = new FOnlineAsyncEventSteamStatsStored(SteamSubsystem, *UserId, CallbackData->m_eResult);
		UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
		AddToOutQueue(NewEvent);
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("Stored steam user stats, but for wrong game! Ignoring."));
	}
}

/**
 * Notification event from Steam that the currently logged in user's
 * stats/achievements data has been stored with the server
 * FROM VALVE: Steam stats for other users are kept in an LRU with a max queue length of 100
 */
class FOnlineAsyncEventSteamStatsUnloaded : public FOnlineAsyncEvent<FOnlineSubsystemSteam>
{
private:
	/** User whose data has been unloaded */
	FUniqueNetIdSteamRef UserId;

	/** Hidden on purpose */
	FOnlineAsyncEventSteamStatsUnloaded() = delete;

public:

	FOnlineAsyncEventSteamStatsUnloaded(FOnlineSubsystemSteam* InSubsystem, const FUniqueNetIdSteam& InUserId) :
		FOnlineAsyncEvent(InSubsystem),
		UserId(InUserId.AsShared())
	{
	}
	
	virtual ~FOnlineAsyncEventSteamStatsUnloaded() 
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncEventSteamStatsUnloaded UserId: %s"), *UserId->ToDebugString());
	}
};

/**
 *	Event triggered from Steam when a previously requested user's stats have been purged in LRU fashion
 * Requesting the data an additional time will bring the data back
 *
 * @param CallbackData All the valid data from Steam related to this event 
 */
void FOnlineAsyncTaskManagerSteam::OnUserStatsUnloaded(UserStatsUnloaded_t* CallbackData)
{
	FOnlineAsyncEventSteamStatsUnloaded* NewEvent = new FOnlineAsyncEventSteamStatsUnloaded(SteamSubsystem, *FUniqueNetIdSteam::Create(CallbackData->m_steamIDUser));
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

/**
 *	Delegate registered with Steam to trigger when the Steam Overlay is activated
 *
 * @param CallbackData - Steam struct containing state of the Steam Overlay
 */
void FOnlineAsyncTaskManagerSteam::OnExternalUITriggered(GameOverlayActivated_t* CallbackData)
{
	FOnlineAsyncEventSteamExternalUITriggered* NewEvent = new FOnlineAsyncEventSteamExternalUITriggered(SteamSubsystem, (CallbackData->m_bActive != 0) ? true : false);
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

/**
 * Notification event from Steam that server session connection has changed state 
 */
class FOnlineAsyncEventSteamServerConnectionState : public FOnlineAsyncEvent<FOnlineSubsystemSteam>
{
	/** Connection state change */
	const EOnlineServerConnectionStatus::Type ConnectionState;

	/** Hidden on purpose */
	FOnlineAsyncEventSteamServerConnectionState() :
		ConnectionState(EOnlineServerConnectionStatus::NotConnected)
	{
	}

public:

	FOnlineAsyncEventSteamServerConnectionState(FOnlineSubsystemSteam* InSubsystem, EOnlineServerConnectionStatus::Type InConnectionState) :
		FOnlineAsyncEvent(InSubsystem),
		ConnectionState(InConnectionState)
	{
	}
	
	virtual ~FOnlineAsyncEventSteamServerConnectionState() 
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncEventSteamServerConnectionState StateChange: %s"), EOnlineServerConnectionStatus::ToString(ConnectionState));	
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override
	{
		Subsystem->TriggerOnConnectionStatusChangedDelegates(Subsystem->GetSubsystemName().ToString(), EOnlineServerConnectionStatus::Normal, ConnectionState);
	}
};

/**
 *	Client API version of the connected to Steam callback (only called in case of a Steam backend disconnect and then reconnect)
 */
void FOnlineAsyncTaskManagerSteam::OnSteamServersConnected(SteamServersConnected_t* CallbackData)
{
	FOnlineAsyncEventSteamServerConnectionState* NewEvent = new FOnlineAsyncEventSteamServerConnectionState(SteamSubsystem, EOnlineServerConnectionStatus::Connected);
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

/**
 *	Client API version of the disconnected to Steam callback 
 */
void FOnlineAsyncTaskManagerSteam::OnSteamServersDisconnected(SteamServersDisconnected_t* CallbackData)
{
	FOnlineAsyncEventSteamServerConnectionState* NewEvent = new FOnlineAsyncEventSteamServerConnectionState(SteamSubsystem, SteamConnectionResult(CallbackData->m_eResult));
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

/**
 * Notification event from Steam that server session has been disconnected with the server list
 */
class FOnlineAsyncEventSteamServerDisconnectedGS : public FOnlineAsyncEvent<FOnlineSubsystemSteam>
{
private:

	/** Callback data */
	SteamServersDisconnected_t CallbackResults;

	/** Hidden on purpose */
	FOnlineAsyncEventSteamServerDisconnectedGS()
	{
	}

public:

	FOnlineAsyncEventSteamServerDisconnectedGS(FOnlineSubsystemSteam* InSubsystem, SteamServersDisconnected_t& InResults) :
		FOnlineAsyncEvent(InSubsystem),
		CallbackResults(InResults)
	{
	}
	
	virtual ~FOnlineAsyncEventSteamServerDisconnectedGS() 
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncEventSteamServerDisconnectedGS Result: %s"), *SteamResultString(CallbackResults.m_eResult));	
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override
	{
		bool bTriggerConnectionStatusUpdate = true;
		FOnlineSessionSteamPtr SessionInt = StaticCastSharedPtr<FOnlineSessionSteam>(Subsystem->GetSessionInterface());
		if (SessionInt.IsValid())
		{
			SessionInt->bSteamworksGameServerConnected = false;
			SessionInt->GameServerSteamId = NULL;

			// Don't trigger the delegates if a DestroySession() call was made
			FNamedOnlineSession* Session = SessionInt->GetGameServerSession();
			if (Session && Session->SessionState == EOnlineSessionState::Destroying)
			{
				bTriggerConnectionStatusUpdate = false;
			}
		}
		
		if (bTriggerConnectionStatusUpdate)
		{
			EOnlineServerConnectionStatus::Type ConnectionState = SteamConnectionResult(CallbackResults.m_eResult);
			Subsystem->TriggerOnConnectionStatusChangedDelegates(Subsystem->GetSubsystemName().ToString(), EOnlineServerConnectionStatus::Normal, ConnectionState);
		}
	}
};

/**
 *	GameServer API version of disconnected from Steam backend callback
 */
void FOnlineAsyncTaskManagerSteam::OnSteamServersDisconnectedGS(SteamServersDisconnected_t* CallbackData)
{
	FOnlineAsyncEventSteamServerDisconnectedGS* NewEvent = new FOnlineAsyncEventSteamServerDisconnectedGS(SteamSubsystem, *CallbackData);
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

/**
 * Notification event from Steam that server login has failed.
 */
class FOnlineAsyncEventSteamServerFailedGS : public FOnlineAsyncEvent<FOnlineSubsystemSteam>
{
private:

	/** Callback data */
	SteamServerConnectFailure_t CallbackResults;

	/** Hidden on purpose */
	FOnlineAsyncEventSteamServerFailedGS()
	{
	}

public:

	FOnlineAsyncEventSteamServerFailedGS(FOnlineSubsystemSteam* InSubsystem, SteamServerConnectFailure_t& InResults) :
		FOnlineAsyncEvent(InSubsystem),
		CallbackResults(InResults)
	{
	}

	virtual ~FOnlineAsyncEventSteamServerFailedGS()
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncEventSteamServerFailedGS Result: %s"), *SteamResultString(CallbackResults.m_eResult));
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override
	{
		if (Subsystem)
		{
			Subsystem->TriggerOnSteamServerLoginCompletedDelegates(false);
		}
	}
};

/**
 *	GameServer API version of disconnected from Steam backend callback
 */
void FOnlineAsyncTaskManagerSteam::OnSteamServersConnectFailureGS(SteamServerConnectFailure_t* CallbackData)
{
	// Only do something if we are no longer retrying to connect with the backend.
	if (!CallbackData->m_bStillRetrying)
	{
		FOnlineAsyncEventSteamServerFailedGS* NewEvent = new FOnlineAsyncEventSteamServerFailedGS(SteamSubsystem, *CallbackData);
		UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
		AddToOutQueue(NewEvent);
	}
}

/**
 * Notification event from Steam that server session has been secured on the backend
 */
class FOnlineAsyncEventSteamServerPolicyResponseGS : public FOnlineAsyncEvent<FOnlineSubsystemSteam>
{
private:

	/** Callback data */
	GSPolicyResponse_t CallbackResults;

	/** Hidden on purpose */
	FOnlineAsyncEventSteamServerPolicyResponseGS()
	{
	}

public:

	FOnlineAsyncEventSteamServerPolicyResponseGS(FOnlineSubsystemSteam* InSubsystem, GSPolicyResponse_t& InResults) :
		FOnlineAsyncEvent(InSubsystem),
		CallbackResults(InResults)
	{
	}
	
	virtual ~FOnlineAsyncEventSteamServerPolicyResponseGS() 
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncEventSteamServerPolicyResponseGS Secure: %d"), CallbackResults.m_bSecure);	
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override
	{
		FOnlineSessionSteamPtr SessionInt = StaticCastSharedPtr<FOnlineSessionSteam>(Subsystem->GetSessionInterface());
		if (SessionInt.IsValid())
		{
			SessionInt->bPolicyResponseReceived = true;
			if (!SessionInt->bSteamworksGameServerConnected || !SessionInt->GameServerSteamId->IsValid())
			{
				UE_LOG_ONLINE(Warning, TEXT("Unexpected GSPolicyResponse callback"));
			}
		}
	}
};

/**
 * Notification event from Steam that server session has been secured
 */
void FOnlineAsyncTaskManagerSteam::OnPolicyResponseGS(GSPolicyResponse_t* CallbackData)
{
	FOnlineAsyncEventSteamServerPolicyResponseGS* NewEvent = new FOnlineAsyncEventSteamServerPolicyResponseGS(SteamSubsystem, *CallbackData);
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

class FOnlineAsyncEventSteamGetTicketForWebApiResponse : public FOnlineAsyncEvent<FOnlineSubsystemSteam>
{
private:
	GetTicketForWebApiResponse_t CallbackResults;

public:
	FOnlineAsyncEventSteamGetTicketForWebApiResponse(FOnlineSubsystemSteam* InSubsystem, const GetTicketForWebApiResponse_t& InResults)
		: FOnlineAsyncEvent(InSubsystem)
		, CallbackResults(InResults)
	{
	}

	virtual ~FOnlineAsyncEventSteamGetTicketForWebApiResponse()
	{
	}

	/**
	*	Get a human readable description of task
	*/
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncEventSteamGetTicketForWebApiResponse Received code %d."), (int32)CallbackResults.m_eResult);
	}

	/**
	* Give the async task a chance to marshal its data back to the game thread
	* Can only be called on the game thread by the async task manager
	*/
	virtual void Finalize() override
	{
		FOnlineAuthSteamPtr AuthInt = StaticCastSharedPtr<FOnlineAuthSteam>(Subsystem->GetAuthInterface());
		if (AuthInt.IsValid())
		{
			FString ResultToken = BytesToHex(CallbackResults.m_rgubTicket, CallbackResults.m_cubTicket);
			AuthInt->OnGetTicketForWebResponse(CallbackResults.m_hAuthTicket, ResultToken);
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("Auth interface is not valid!"));
		}
	}
};

void FOnlineAsyncTaskManagerSteam::OnGetTicketForWebApiResponse(GetTicketForWebApiResponse_t* CallbackData)
{
	FOnlineAsyncEventSteamGetTicketForWebApiResponse* NewEvent = new FOnlineAsyncEventSteamGetTicketForWebApiResponse(SteamSubsystem, *CallbackData);
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

class FOnlineAsyncEventSteamAuthenticationResponse : public FOnlineAsyncEvent<FOnlineSubsystemSteam>
{
private:
	ValidateAuthTicketResponse_t CallbackResults;
	bool bIsServer;

	FOnlineAsyncEventSteamAuthenticationResponse()
	{
	}

public:
	FOnlineAsyncEventSteamAuthenticationResponse(FOnlineSubsystemSteam* InSubsystem, const ValidateAuthTicketResponse_t& InResults, bool bInServerCall) :
		FOnlineAsyncEvent(InSubsystem),
		CallbackResults(InResults),
		bIsServer(bInServerCall)
	{
	}

	virtual ~FOnlineAsyncEventSteamAuthenticationResponse()
	{
	}

	/**
	*	Get a human readable description of task
	*/
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncEventSteamAuthenticationResponse Received code %d. Is server? %d"), (int32)CallbackResults.m_eAuthSessionResponse, bIsServer);
	}

	/**
	* Give the async task a chance to marshal its data back to the game thread
	* Can only be called on the game thread by the async task manager
	*/
	virtual void Finalize() override
	{
		FOnlineAuthSteamPtr AuthInt = StaticCastSharedPtr<FOnlineAuthSteam>(Subsystem->GetAuthInterface());
		if (AuthInt.IsValid())
		{
			AuthInt->OnAuthResult(*FUniqueNetIdSteam::Create(CallbackResults.m_SteamID), CallbackResults.m_eAuthSessionResponse);
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("Auth interface is not valid!"));
		}
	}
};

/**
* Notification event from Steam regarding a ticket authentication response for server
*/
void FOnlineAsyncTaskManagerSteam::OnAuthenticationResponseGS(ValidateAuthTicketResponse_t* CallbackData)
{
	FOnlineAsyncEventSteamAuthenticationResponse* NewEvent = new FOnlineAsyncEventSteamAuthenticationResponse(SteamSubsystem, *CallbackData, true);
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

/**
* Notification event from Steam regarding a ticket authentication response for clients
*/
void FOnlineAsyncTaskManagerSteam::OnAuthenticationResponse(ValidateAuthTicketResponse_t* CallbackData)
{
	FOnlineAsyncEventSteamAuthenticationResponse* NewEvent = new FOnlineAsyncEventSteamAuthenticationResponse(SteamSubsystem, *CallbackData, false);
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

/**
 * Notification event from Steam that a P2P connection has failed
 */
class FOnlineAsyncEventSteamShutdown : public FOnlineAsyncEvent<FOnlineSubsystemSteam>
{
	FOnlineAsyncEventSteamShutdown() :
		FOnlineAsyncEvent(NULL)
	{
	}

public:

	FOnlineAsyncEventSteamShutdown(FOnlineSubsystemSteam* InSubsystem) :
		FOnlineAsyncEvent(InSubsystem)
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncEventSteamShutdown shutdown received."));	
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override
	{
		FPlatformMisc::RequestExit(false);
	}
};

/**
 *	Delegate registered with Steam to trigger when Steam is shutting down
 *
 * @param CallbackData - Steam struct containing shutdown information
 */
void FOnlineAsyncTaskManagerSteam::OnSteamShutdown(SteamShutdown_t* CallbackData)
{
	FOnlineAsyncEventSteamShutdown* NewEvent = new FOnlineAsyncEventSteamShutdown(SteamSubsystem);
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

/**
 * Notification event from Steam that rich presence has updated
 */
class FOnlineAsyncEventSteamRichPresenceUpdate : public FOnlineAsyncEvent<FOnlineSubsystemSteam>
{
	FOnlineAsyncEventSteamRichPresenceUpdate() = delete;

	FUniqueNetIdSteamRef TargetSteamId;

public:

	FOnlineAsyncEventSteamRichPresenceUpdate(FOnlineSubsystemSteam* InSubsystem, CSteamID InSteamId) :
		FOnlineAsyncEvent(InSubsystem),
		TargetSteamId(FUniqueNetIdSteam::Create(InSteamId))
	{
	}

	FOnlineAsyncEventSteamRichPresenceUpdate(FOnlineSubsystemSteam* InSubsystem, uint64 InSteamId) :
		FOnlineAsyncEvent(InSubsystem),
		TargetSteamId(FUniqueNetIdSteam::Create(InSteamId))
	{
	}

	/**
	 * Get a human readable description of task
	 */
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncEventSteamRichPresenceUpdate got new information about user %s"), *TargetSteamId->ToString());
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override
	{
		FOnlinePresenceSteamPtr PresenceInterface = StaticCastSharedPtr<FOnlinePresenceSteam>(Subsystem->GetPresenceInterface());
		if (PresenceInterface.IsValid())
		{
			PresenceInterface->UpdatePresenceForUser(*TargetSteamId);
		}
	}
};

/**
 * Delegate registered with Steam to trigger when Steam gets updates about user rich presence
 *
 * @param CallbackData - Steam struct containing user that got their data updated
 */
void FOnlineAsyncTaskManagerSteam::OnRichPresenceUpdate(FriendRichPresenceUpdate_t* CallbackData)
{
	FOnlineAsyncEventSteamRichPresenceUpdate* NewEvent = new FOnlineAsyncEventSteamRichPresenceUpdate(SteamSubsystem, CallbackData->m_steamIDFriend);
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

/**
 * Delegate registered with Steam to trigger when Steam gets updates about user rich presence
 *
 * @param CallbackData - Steam struct containing user that got their data updated
 */
void FOnlineAsyncTaskManagerSteam::OnFriendStatusUpdate(PersonaStateChange_t* CallbackData)
{
	int ChangedData = CallbackData->m_nChangeFlags;
	// Licensees can feel free to expand on this by adding their own watch events as well.
	int RichPresenceWatchedEvents = (k_EPersonaChangeGameServer | k_EPersonaChangeGamePlayed | k_EPersonaChangeStatus | k_EPersonaChangeGoneOffline | k_EPersonaChangeComeOnline);
	
	if (ChangedData & RichPresenceWatchedEvents)
	{
		FOnlineAsyncEventSteamRichPresenceUpdate* NewEvent = new FOnlineAsyncEventSteamRichPresenceUpdate(SteamSubsystem, CallbackData->m_ulSteamID);
		UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
		AddToOutQueue(NewEvent);
	}
}
