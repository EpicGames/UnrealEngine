// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemSteamPrivate.h" // IWYU pragma: keep
#include "OnlineAsyncTaskManager.h"
#include "OnlineSubsystemSteamPackage.h"

/**
 * Base class that holds a delegate to fire when a given async task is complete
 */
class FOnlineAsyncTaskSteam : public FOnlineAsyncTaskBasic<class FOnlineSubsystemSteam>
{
PACKAGE_SCOPE:

	/** Unique handle for the Steam async call initiated */
	SteamAPICall_t CallbackHandle;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteam() :
		FOnlineAsyncTaskBasic(NULL),
		CallbackHandle(k_uAPICallInvalid)
	{
	}

public:

	FOnlineAsyncTaskSteam(class FOnlineSubsystemSteam* InSteamSubsystem, SteamAPICall_t InCallbackHandle) :
		FOnlineAsyncTaskBasic(InSteamSubsystem),	
		CallbackHandle(InCallbackHandle)
	{
	}

	virtual ~FOnlineAsyncTaskSteam()
	{
	}
};

/**
 *	Steam version of the async task manager to register the various Steam callbacks with the engine
 */
class FOnlineAsyncTaskManagerSteam : public FOnlineAsyncTaskManager
{
private:

	/** Delegate registered with Steam to trigger when a user (client API) is connected to the Steam servers  (usually don't get this because we're already connected externally) */
	STEAM_CALLBACK(FOnlineAsyncTaskManagerSteam, OnSteamServersConnected, SteamServersConnected_t, OnSteamServersConnectedCallback);
	/** Delegate registered with Steam to trigger when a user (client API) is disconnected from the Steam servers.  All realtime services are disabled until a connect event above */
	STEAM_CALLBACK(FOnlineAsyncTaskManagerSteam, OnSteamServersDisconnected, SteamServersDisconnected_t, OnSteamServersDisconnectedCallback);
	/** Delegate registered with Steam to trigger when a server (gameserver API) is disconnected from Steam.  Will occur on DestroySession() */
	STEAM_GAMESERVER_CALLBACK(FOnlineAsyncTaskManagerSteam, OnSteamServersDisconnectedGS, SteamServersDisconnected_t, OnSteamServersDisconnectedGSCallback);
	/** Delegate registered with Steam to trigger when a server (gameserver API) connection fails. */
	STEAM_GAMESERVER_CALLBACK(FOnlineAsyncTaskManagerSteam, OnSteamServersConnectFailureGS, SteamServerConnectFailure_t, OnSteamServersConnectFailureGSCallback);
	/** Delegate registered with Steam to trigger when a server (gameserver API) is properly connected to Steam servers, logon isn't complete until this occurs */
	STEAM_GAMESERVER_CALLBACK(FOnlineAsyncTaskManagerSteam, OnPolicyResponseGS, GSPolicyResponse_t, OnPolicyResponseGSCallback);

	/** Delegate registered with Steam to trigger when a user (client API) gets a response for a GetTicketForWebApiResponse request */
	STEAM_CALLBACK(FOnlineAsyncTaskManagerSteam, OnGetTicketForWebApiResponse, GetTicketForWebApiResponse_t, OnGetTicketForWebApiResponseCallback);
	/** Delegate registered with Steam to trigger when a server (gameserver API) gets a response for a Steam session authentication ticket request */
	STEAM_GAMESERVER_CALLBACK(FOnlineAsyncTaskManagerSteam, OnAuthenticationResponseGS, ValidateAuthTicketResponse_t, OnAuthenticationResponseGSCallback);
	/** Delegate registered with Steam to trigger when a user (client API) gets a response for a Steam session authentication ticket request */
	STEAM_CALLBACK(FOnlineAsyncTaskManagerSteam, OnAuthenticationResponse, ValidateAuthTicketResponse_t, OnAuthenticationResponseCallback);

	/** Delegate registered with Steam to trigger when a user accepts an invite request to a non lobby session */
	STEAM_CALLBACK(FOnlineAsyncTaskManagerSteam, OnInviteAccepted, GameRichPresenceJoinRequested_t, OnInviteAcceptedCallback);
	/** Delegate registered with Steam to trigger when a user attempts JIP (via Steam client) or accepts an invite request (via Steam client) to a lobby session */
	STEAM_CALLBACK(FOnlineAsyncTaskManagerSteam, OnLobbyInviteAccepted, GameLobbyJoinRequested_t, OnLobbyInviteAcceptedCallback);
	/** Delegate registered with Steam to trigger when a user joins a lobby (successfully or not) */
	STEAM_CALLBACK(FOnlineAsyncTaskManagerSteam, OnLobbyEnter, LobbyEnter_t, OnLobbyEnterCallback);
	/** Delegate registered with Steam to trigger when a lobby chat update occurs (usually users joining/leaving lobby) */
	STEAM_CALLBACK(FOnlineAsyncTaskManagerSteam, OnLobbyChatUpdate, LobbyChatUpdate_t, OnLobbyChatUpdateCallback);
	/** 
     * Delegate registered with Steam to trigger when a lobby update occurs
	 * As host (or any client joined to the lobby), any call to SetLobbyData will trigger this
	 * As client searching for lobbies, once per call to RequestLobbyData
     */
	STEAM_CALLBACK(FOnlineAsyncTaskManagerSteam, OnLobbyDataUpdate, LobbyDataUpdate_t, OnLobbyDataUpdateCallback);
	/** Delegate registered with Steam to trigger when the currently logged in user receives their stats from RequestStats() */
	STEAM_CALLBACK(FOnlineAsyncTaskManagerSteam, OnUserStatsReceived, UserStatsReceived_t, OnUserStatsReceivedCallback);
	/** Delegate registered with Steam to trigger when the currently logged in user has their stats stored */
	STEAM_CALLBACK(FOnlineAsyncTaskManagerSteam, OnUserStatsStored, UserStatsStored_t, OnUserStatsStoredCallback);
	/** Delegate registered with Steam to trigger when any previously requested user's stat are unloaded by the system */
	STEAM_CALLBACK(FOnlineAsyncTaskManagerSteam, OnUserStatsUnloaded, UserStatsUnloaded_t, OnUserStatsUnloadedCallback);
	/** Delegate registered with Steam to trigger when the Steam Overlay is activated */
	STEAM_CALLBACK(FOnlineAsyncTaskManagerSteam, OnExternalUITriggered, GameOverlayActivated_t, OnExternalUITriggeredCallback);
	/** Delegate registered with Steam to trigger when Steam is shutting down */
	STEAM_CALLBACK(FOnlineAsyncTaskManagerSteam, OnSteamShutdown, SteamShutdown_t, OnSteamShutdownCallback);
	/** Delegate registered with Steam to trigger when Steam gets updates about user rich presence */
	STEAM_CALLBACK(FOnlineAsyncTaskManagerSteam, OnRichPresenceUpdate, FriendRichPresenceUpdate_t, OnRichPresenceUpdateCallback);
	/** Delegate registered with Steam to trigger when Steam gets updates about friends status */
	STEAM_CALLBACK(FOnlineAsyncTaskManagerSteam, OnFriendStatusUpdate, PersonaStateChange_t, OnFriendStatusUpdateCallback);

	//GameServerChangeRequested_t

protected:

	/** Cached reference to the main online subsystem */
	class FOnlineSubsystemSteam* SteamSubsystem;

public:

	FOnlineAsyncTaskManagerSteam(class FOnlineSubsystemSteam* InOnlineSubsystem) :
		OnSteamServersConnectedCallback(this, &FOnlineAsyncTaskManagerSteam::OnSteamServersConnected),
		OnSteamServersDisconnectedCallback(this, &FOnlineAsyncTaskManagerSteam::OnSteamServersDisconnected),
		OnSteamServersDisconnectedGSCallback(this, &FOnlineAsyncTaskManagerSteam::OnSteamServersDisconnectedGS),
		OnSteamServersConnectFailureGSCallback(this, &FOnlineAsyncTaskManagerSteam::OnSteamServersConnectFailureGS),
		OnPolicyResponseGSCallback(this, &FOnlineAsyncTaskManagerSteam::OnPolicyResponseGS),
		OnGetTicketForWebApiResponseCallback(this, &FOnlineAsyncTaskManagerSteam::OnGetTicketForWebApiResponse),
		OnAuthenticationResponseGSCallback(this, &FOnlineAsyncTaskManagerSteam::OnAuthenticationResponseGS),
		OnAuthenticationResponseCallback(this, &FOnlineAsyncTaskManagerSteam::OnAuthenticationResponse),
		OnInviteAcceptedCallback(this, &FOnlineAsyncTaskManagerSteam::OnInviteAccepted),
		OnLobbyInviteAcceptedCallback(this, &FOnlineAsyncTaskManagerSteam::OnLobbyInviteAccepted),
		OnLobbyEnterCallback(this, &FOnlineAsyncTaskManagerSteam::OnLobbyEnter),
		OnLobbyChatUpdateCallback(this, &FOnlineAsyncTaskManagerSteam::OnLobbyChatUpdate),
		OnLobbyDataUpdateCallback(this, &FOnlineAsyncTaskManagerSteam::OnLobbyDataUpdate),
		OnUserStatsReceivedCallback(this, &FOnlineAsyncTaskManagerSteam::OnUserStatsReceived),
		OnUserStatsStoredCallback(this, &FOnlineAsyncTaskManagerSteam::OnUserStatsStored),
		OnUserStatsUnloadedCallback(this, &FOnlineAsyncTaskManagerSteam::OnUserStatsUnloaded),
		OnExternalUITriggeredCallback(this, &FOnlineAsyncTaskManagerSteam::OnExternalUITriggered),
		OnSteamShutdownCallback(this, &FOnlineAsyncTaskManagerSteam::OnSteamShutdown),
		OnRichPresenceUpdateCallback(this, &FOnlineAsyncTaskManagerSteam::OnRichPresenceUpdate),
		OnFriendStatusUpdateCallback(this, &FOnlineAsyncTaskManagerSteam::OnFriendStatusUpdate),
		SteamSubsystem(InOnlineSubsystem)
	{
	}

	~FOnlineAsyncTaskManagerSteam() 
	{
	}

	// FOnlineAsyncTaskManager
	virtual void OnlineTick() override;

	// FOnlineAsyncTaskManagerSteam
};


