// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineAuthHandlerSteam.h"
#include "OnlineAuthInterfaceSteam.h"
#include "OnlineAuthInterfaceUtilsSteam.h"
#include <steam/isteamuser.h>

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnlineAuthHandlerSteam)

enum class ESteamAuthMsgType : uint8
{
	None = 0,
	Auth,
	Result,
	ResendKey,
	ResendResult,
	Max
};

struct FSteamAuthInfoData
{
	FSteamAuthInfoData() : Type(ESteamAuthMsgType::None) {}
	FSteamAuthInfoData(ESteamAuthMsgType InType) : Type(InType) {}
	virtual ~FSteamAuthInfoData() {}

	ESteamAuthMsgType Type;

	virtual void SerializeData(FArchive& Ar)
	{
		Ar << Type;
	}

	friend FArchive& operator<<(FArchive& Ar, FSteamAuthInfoData& AuthData)
	{
		AuthData.SerializeData(Ar);
		return Ar;
	}
};

struct FSteamAuthResult : public FSteamAuthInfoData
{
	FSteamAuthResult() : FSteamAuthInfoData(ESteamAuthMsgType::Result), bWasSuccess(false) {}
	virtual ~FSteamAuthResult() {}
	bool bWasSuccess;

	virtual void SerializeData(FArchive& Ar) override
	{
		FSteamAuthInfoData::SerializeData(Ar);
		Ar << bWasSuccess;
	}

	friend FArchive& operator<<(FArchive& Ar, FSteamAuthResult& AuthData)
	{
		AuthData.SerializeData(Ar);
		return Ar;
	}
};

struct FSteamAuthUserData : public FSteamAuthInfoData
{
	FSteamAuthUserData()
		: FSteamAuthInfoData(ESteamAuthMsgType::Auth)
		, SteamId(FUniqueNetIdSteam::EmptyId())
	{}

	virtual ~FSteamAuthUserData() {}
	FString AuthKey;
	FUniqueNetIdSteamRef SteamId;

	virtual void SerializeData(FArchive& Ar) override
	{
		FSteamAuthInfoData::SerializeData(Ar);
		if (Ar.IsLoading())
		{
			// Create a new one to avoid changing the hash of a pre-existing NetId, which would break any FUniqueNetIdSet/TUniqueNetIdMap containing the pre-existing NetId.
			SteamId = FUniqueNetIdSteam::Create();
		}
		Ar << AuthKey << ConstCastSharedRef<FUniqueNetIdSteam>(SteamId)->UniqueNetId;
	}

	friend FArchive& operator<<(FArchive& Ar, FSteamAuthUserData& AuthData)
	{
		AuthData.SerializeData(Ar);
		return Ar;
	}
};


// Easy resaving key for easy usage in testing
#if !UE_BUILD_SHIPPING
FString ReusableKey;
#endif

/* Steam Auth Packet Handler */
FSteamAuthHandlerComponent::FSteamAuthHandlerComponent() :
	AuthInterface(nullptr),
	SteamUserPtr(SteamUser()),
	State(ESteamAuthHandlerState::Uninitialized),
	bIsEnabled(true),
	LastTimestamp(0.0f),
	TicketHandle(k_HAuthTicketInvalid),
	SteamId(SteamUserPtr ? FUniqueNetIdSteam::Create(SteamUserPtr->GetSteamID()) : FUniqueNetIdSteam::EmptyId())
{
	SetActive(true);
	bRequiresHandshake = true;

	FOnlineSubsystemSteam* OSS = (FOnlineSubsystemSteam*)(IOnlineSubsystem::Get(STEAM_SUBSYSTEM));
	if (OSS != nullptr)
	{
		AuthInterface = OSS->GetAuthInterface();
		if (!AuthInterface.IsValid() || !AuthInterface->IsSessionAuthEnabled())
		{
			bIsEnabled = false;
		}
	}
	else
	{
		bIsEnabled = false;
	}
}

FSteamAuthHandlerComponent::~FSteamAuthHandlerComponent()
{
	if (!bIsEnabled || !AuthInterface.IsValid())
	{
		return;
	}

	if (Handler->Mode == UE::Handler::Mode::Client)
	{
		AuthInterface->RevokeTicket(TicketHandle);
	}
	else
	{
		AuthInterface->RemoveUser(*SteamId);
	}
}

void FSteamAuthHandlerComponent::CountBytes(FArchive& Ar) const
{
	HandlerComponent::CountBytes(Ar);

	const SIZE_T SizeOfThis = sizeof(*this) - sizeof(HandlerComponent);
	Ar.CountBytes(SizeOfThis, SizeOfThis);

	UserTicket.CountBytes(Ar);
}

void FSteamAuthHandlerComponent::Initialize()
{
	if (!AuthInterface.IsValid() || !AuthInterface->IsSessionAuthEnabled())
	{
		UE_LOG_ONLINE(Warning, TEXT("AUTH HANDLER: Deactivating due to missing requirements"));
		bIsEnabled = false;
		if (Handler != nullptr)
		{
			SetComponentReady();
		}
		else
		{
			SetActive(false);
		}
	}
}

void FSteamAuthHandlerComponent::NotifyHandshakeBegin()
{
	if (!bIsEnabled)
	{
		return;
	}

	if (Handler->Mode == UE::Handler::Mode::Client)
	{
		SendAuthKey(true);
	}
	else
	{
		SetState(ESteamAuthHandlerState::WaitingForKey);
		LastTimestamp = FPlatformTime::Seconds();
	}
}

void FSteamAuthHandlerComponent::SendAuthKey(bool bGenerateNewKey)
{
	FBitWriter AuthDataPacket((sizeof(FSteamAuthUserData) + FOnlineAuthSteam::GetMaxTicketSizeInBytes()) * 8 + 1);
	FSteamAuthUserData UserData;
	UserData.SteamId = SteamId;

	if (bGenerateNewKey || TicketHandle == k_HAuthTicketInvalid)
	{
		UserTicket = AuthInterface->GetAuthTicket(TicketHandle);
	}

#if !UE_BUILD_SHIPPING
	if (ReusableKey.IsEmpty())
	{
		ReusableKey = UserTicket;
	}
	else if (AuthInterface->bReuseKey)
	{
		UserTicket = ReusableKey;
	}

	if (AuthInterface->bNeverSendKey)
	{
		SetState(ESteamAuthHandlerState::SentAuthKey);
		return;
	}

	if (AuthInterface->bBadKey)
	{
		UserTicket = TEXT("THIS IS A BAD STEAM KEY");
	}

	if (AuthInterface->bSendBadId)
	{
		UserData.SteamId = FUniqueNetIdSteam::EmptyId();
	}
#endif

	UserData.AuthKey = UserTicket;

	AuthDataPacket.WriteBit(1);
	AuthDataPacket << UserData;
	SendPacket(AuthDataPacket);

	SetState(ESteamAuthHandlerState::SentAuthKey);
	UE_LOG_ONLINE(Log, TEXT("AUTH HANDLER: Sending auth key"));
}

bool FSteamAuthHandlerComponent::SendAuthResult()
{
	// This function is safe to call multiple times. If we're already in progress, we let the user go through.
	bool AuthStatusResult = AuthInterface->AuthenticateUser(*SteamId);

	FSteamAuthResult AllowedPacket;
	FBitWriter ResultPacketWriter(sizeof(FSteamAuthResult) * 8 + 1, true);
	ResultPacketWriter.WriteBit(1);

	AllowedPacket.bWasSuccess = AuthStatusResult;
	ResultPacketWriter << AllowedPacket;

	SendPacket(ResultPacketWriter);
	
	UE_LOG_ONLINE(Log, TEXT("AUTH HANDLER: Sending auth result to user %s with flag success? %d"), *SteamId->ToString(), AuthStatusResult);

	return AuthStatusResult;
}

void FSteamAuthHandlerComponent::SendPacket(FBitWriter& OutboundPacket)
{
#if !UE_BUILD_SHIPPING
	if (AuthInterface->bBadWrite)
	{
		OutboundPacket.SetError();
	}

	if (AuthInterface->bDropAll)
	{
		return;
	}

	if (AuthInterface->bRandomDrop && FMath::RandBool() == false)
	{
		UE_LOG_ONLINE(Warning, TEXT("AUTH HANDLER: Random packet was dropped!"));
		return;
	}
#endif

	FOutPacketTraits Traits;

	Handler->SendHandlerPacket(this, OutboundPacket, Traits);
	LastTimestamp = FPlatformTime::Seconds();
}

void FSteamAuthHandlerComponent::RequestResend()
{
	FBitWriter ResendWriter(sizeof(FSteamAuthInfoData) * 8 + 1);
	FSteamAuthInfoData ResendingPacket;

	ResendWriter.WriteBit(1);

	// Steam Auth is so simplistic that we really only have two messages we need to handle.
	ResendingPacket.Type = (Handler->Mode == UE::Handler::Mode::Server) ? 
		ESteamAuthMsgType::ResendKey : ESteamAuthMsgType::ResendResult;

	ResendWriter << ResendingPacket;
	SendPacket(ResendWriter);
}

bool FSteamAuthHandlerComponent::IsValid() const
{
	return bIsEnabled;
}

void FSteamAuthHandlerComponent::Incoming(FBitReader& Packet)
{
	bool bForSteamAuth = !!Packet.ReadBit() && !Packet.IsError();
	if (!bIsEnabled || !AuthInterface.IsValid() || !bForSteamAuth)
	{
		return;
	}

#if !UE_BUILD_SHIPPING
	if (AuthInterface->bDropAll)
	{
		Packet.SetError();
		return;
	}
#endif

	// Save our position so we can parse the header.
	FBitReaderMark PacketMarker(Packet);
	FSteamAuthInfoData Header;

	// Try to grab information from the packet.
	Packet << Header;

	if (Packet.IsError())
	{
		UE_LOG_ONLINE(Error, TEXT("AUTH HANDLER: Incoming steam auth packet could not be properly serialized."));
		return;
	}

	// Reset to actually read the data.
	PacketMarker.Pop(Packet);

	if (State == ESteamAuthHandlerState::WaitingForKey && Header.Type == ESteamAuthMsgType::Auth)
	{
		FSteamAuthUserData AuthData;
		Packet << AuthData;

		if (Packet.IsError())
		{
			// Really this is if we somehow overflow and cannot fit the packet.
			UE_LOG_ONLINE(Warning, TEXT("AUTH HANDLER: Packet was marked as error after serializing"));
			return;
		}

		SteamId = AuthData.SteamId;
		if (!SteamId->IsValid())
		{
			UE_LOG_ONLINE(Error, TEXT("AUTH HANDLER: Got an invalid steamid"));
			AuthInterface->ExecuteResultDelegate(*SteamId, false, ESteamAuthResponseCode::NotConnectedToSteam);
			Packet.SetError();
			return;
		}

		FOnlineAuthSteam::SharedAuthUserSteamPtr TargetUser = AuthInterface->GetOrCreateUser(*SteamId);
		if (!TargetUser.IsValid())
		{
			UE_LOG_ONLINE(Error, TEXT("AUTH HANDLER: Could not create user listing for %s"), *SteamId->ToString());
			AuthInterface->ExecuteResultDelegate(*SteamId, false, ESteamAuthResponseCode::FailedToCreateUser);
			Packet.SetError();
			return;
		}

		TargetUser->SetKey(AuthData.AuthKey);

		if (!SendAuthResult())
		{
			AuthInterface->MarkPlayerForKick(*SteamId);
		}

		SetComponentReady();
	}
	else if (State == ESteamAuthHandlerState::SentAuthKey)
	{
		if (Header.Type == ESteamAuthMsgType::Result)
		{
			FSteamAuthResult AuthResult;
			Packet << AuthResult;

			UE_LOG_ONLINE(Verbose, TEXT("AUTH HANDLER: Got result from server, was success? %d"), AuthResult.bWasSuccess);
			// Regardless of success, we need to ready up, this allows kicks to work.
			SetComponentReady();
		}
		else if (Header.Type == ESteamAuthMsgType::ResendKey)
		{
			UE_LOG_ONLINE(Log, TEXT("AUTH HANDLER: Server requested us to resend our key."));
			SendAuthKey(false);
		}
	}
	else if (Handler && Handler->Mode == UE::Handler::Mode::Server && Header.Type == ESteamAuthMsgType::ResendResult)
	{
		if (State == ESteamAuthHandlerState::Initialized)
		{
			UE_LOG_ONLINE(Log, TEXT("AUTH HANDLER: Got request from %s to resend result"), *SteamId->ToString());
			SendAuthResult();
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("AUTH HANDLER: User has not sent ticket and requesting results."));
			RequestResend();
		}
	}
}

void FSteamAuthHandlerComponent::Outgoing(FBitWriter& Packet, FOutPacketTraits& Traits)
{
#if !UE_BUILD_SHIPPING
	if (AuthInterface.IsValid() && AuthInterface->bDropAll)
	{
		Packet.SetError();
		return;
	}
#endif

	FBitWriter NewPacket(Packet.GetNumBits() + 1, true);

	// We want to specify this is not a Steam auth packet.
	NewPacket.WriteBit(0);
	NewPacket.SerializeBits(Packet.GetData(), Packet.GetNumBits());

	Packet = MoveTemp(NewPacket);
}

void FSteamAuthHandlerComponent::Tick(float DeltaTime)
{
	// Don't do anything if we're not enabled or not ready.
	// Alternatively, if we're already finished then just don't do anything here either
	if (!bIsEnabled || State == ESteamAuthHandlerState::Initialized || !Handler)
	{
		return;
	}

	float CurTime = FPlatformTime::Seconds();
	if (LastTimestamp != 0.0 && CurTime - LastTimestamp > 2.0f)
	{
		RequestResend();
	}
}

int32 FSteamAuthHandlerComponent::GetReservedPacketBits() const
{
	// Add a singular bit to figure out if the message is for Steam Auth
	return 1;
}

void FSteamAuthHandlerComponent::SetComponentReady()
{
	if (State != ESteamAuthHandlerState::Initialized)
	{
		SetState(ESteamAuthHandlerState::Initialized);
		Initialized();
	}
}

bool FSteamAuthHandlerComponent::SupportsParallelConnectionOutgoing() const
{
	return true;
}

/* Module handler */
USteamAuthComponentModuleInterface::USteamAuthComponentModuleInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

TSharedPtr<HandlerComponent> USteamAuthComponentModuleInterface::CreateComponentInstance(FString& Options)
{
	return MakeShareable(new FSteamAuthHandlerComponent);
}
