// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemSteam.h"
#include "OnlineSubsystemSteamTypes.h"
#include "PacketHandler.h"
#include "HandlerComponentFactory.h"
#include "OnlineAuthHandlerSteam.generated.h"

struct FBitReader;
struct FBitWriter;
struct FOutPacketTraits;

class FSteamAuthHandlerComponent : public HandlerComponent
{
public:
	FSteamAuthHandlerComponent();
	virtual ~FSteamAuthHandlerComponent();
	virtual void CountBytes(FArchive& Ar) const override;
	virtual void Initialize() override;
	virtual void NotifyHandshakeBegin() override;

	virtual bool IsValid() const override;

	virtual void Incoming(FBitReader& Packet) override;
	virtual void Outgoing(FBitWriter& Packet, FOutPacketTraits& Traits) override;

	virtual void Tick(float DeltaTime) override;

	virtual int32 GetReservedPacketBits() const override;

	virtual bool SupportsParallelConnectionOutgoing() const override;

protected:
	enum class ESteamAuthHandlerState : uint8
	{
		Uninitialized,
		WaitingForKey, /* Server jumps to this immediately */
		SentAuthKey, /* Client hops here, should their key work, they get another message that allows them to continue */
		Initialized
	};

	void SetState(ESteamAuthHandlerState NewState) { State = NewState; }
	void SetComponentReady();
	void SendAuthKey(bool bGenerateNewKey);
	bool SendAuthResult();
	void SendPacket(FBitWriter& OutboundPacket);
	void RequestResend();

	FOnlineAuthSteamPtr AuthInterface;
	class ISteamUser* SteamUserPtr;

	ESteamAuthHandlerState State;
	bool bIsEnabled;
	float LastTimestamp;

	FString UserTicket;
	uint32 TicketHandle;
	FUniqueNetIdSteamRef SteamId;
};

UCLASS()
class USteamAuthComponentModuleInterface : public UHandlerComponentFactory
{
	GENERATED_UCLASS_BODY()

public:
	virtual TSharedPtr<HandlerComponent> CreateComponentInstance(FString& Options) override;
};
