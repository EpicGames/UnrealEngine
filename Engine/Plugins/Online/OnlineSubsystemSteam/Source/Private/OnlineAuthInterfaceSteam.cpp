// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineAuthInterfaceSteam.h"
#include "OnlineAuthInterfaceUtilsSteam.h"
#include "OnlineSubsystemSteamTypes.h"


// Determining if we need to be disabled or not
#include "Misc/ConfigCacheIni.h"
#include <steam/isteamgameserver.h>
#include <steam/isteamuser.h>

// Steam tells us this number in documentation, however there's no define within the SDK
#define STEAM_AUTH_MAX_TICKET_LENGTH_IN_BYTES 1024

#if WITH_ENGINE
#include "OnlineSubsystemUtils.h"
#include "PacketHandler.h"

// Headers needed to kick users.
#include "GameFramework/GameModeBase.h"
#include "GameFramework/GameSession.h"
#include "GameFramework/PlayerState.h"
#include "GameFramework/PlayerController.h"
#endif //WITH_ENGINE

FOnlineAuthSteam::FOnlineAuthSteam(FOnlineSubsystemSteam* InSubsystem, FOnlineAuthSteamUtilsPtr InAuthUtils) :
	SteamUserPtr(SteamUser()),
	SteamServerPtr(SteamGameServer()),
	SteamSubsystem(InSubsystem),
	AuthUtils(InAuthUtils),
	bEnabled(false),
	bBadKey(false),
	bReuseKey(false),
	bBadWrite(false),
	bDropAll(false),
	bRandomDrop(false),
	bNeverSendKey(false),
	bSendBadId(false)
{

#if WITH_ENGINE
	const FString SteamModuleName(TEXT("SteamAuthComponentModuleInterface"));
	if (!PacketHandler::DoesAnyProfileHaveComponent(SteamModuleName))
	{
		// Pull the components to see if there's anything we can use.
		TArray<FString> ComponentList;
		GConfig->GetArray(TEXT("PacketHandlerComponents"), TEXT("Components"), ComponentList, GEngineIni);

		// Check if Steam Auth is enabled anywhere.
		for (FString CompStr : ComponentList)
		{
			if (CompStr.Contains(SteamModuleName))
			{
				bEnabled = true;
				break;
			}
		}
	}
	else
#endif //WITH_ENGINE
	{
		bEnabled = true;
	}

	if (bEnabled)
	{
		UE_LOG_ONLINE(Log, TEXT("AUTH: Steam Auth Enabled"));
	}
}

FOnlineAuthSteam::FOnlineAuthSteam() :
	SteamUserPtr(nullptr),
	SteamServerPtr(nullptr),
	SteamSubsystem(nullptr),
	AuthUtils(nullptr),
	bEnabled(false),
	bBadKey(false),
	bReuseKey(false),
	bBadWrite(false),
	bDropAll(false),
	bRandomDrop(false),
	bNeverSendKey(false),
	bSendBadId(false)
{
}

FOnlineAuthSteam::~FOnlineAuthSteam()
{
	RevokeAllTickets();
}

uint32 FOnlineAuthSteam::GetMaxTicketSizeInBytes()
{
	return STEAM_AUTH_MAX_TICKET_LENGTH_IN_BYTES;
}

FString FOnlineAuthSteam::GetAuthTicket(uint32& AuthTokenHandle)
{
	FString ResultToken;
	AuthTokenHandle = k_HAuthTicketInvalid;
	// Double check they are properly logged in
	if (SteamUserPtr != nullptr && SteamUserPtr->BLoggedOn())
	{
		uint8 AuthToken[STEAM_AUTH_MAX_TICKET_LENGTH_IN_BYTES];
		uint32 AuthTokenSize = 0;
		AuthTokenHandle = SteamUserPtr->GetAuthSessionTicket(AuthToken, UE_ARRAY_COUNT(AuthToken), &AuthTokenSize, nullptr);
		if (AuthTokenHandle != k_HAuthTicketInvalid && AuthTokenSize > 0)
		{
			ResultToken = BytesToHex(AuthToken, AuthTokenSize);
			SteamTicketHandles.AddUnique(AuthTokenHandle);
			UE_LOG_ONLINE(Verbose, TEXT("AUTH: Generated steam authticket %s handle %d"), OSS_REDACT(*ResultToken), AuthTokenHandle);
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("AUTH: Failed to create Steam auth session ticket"));
		}
	}
	return ResultToken;
}

void FOnlineAuthSteam::GetAuthTicketForWebApi(const FString& RemoteServiceIdentity, FOnGetAuthTicketForWebApiCompleteDelegate CompletionDelegate)
{
	if (SteamUserPtr != NULL && SteamUserPtr->BLoggedOn())
	{
		HAuthTicket TicketHandle = SteamUserPtr->GetAuthTicketForWebApi((const char*)StringCast<UTF8CHAR>(*RemoteServiceIdentity).Get());
		ActiveAuthTicketForWebApiRequests.Emplace(TicketHandle, CompletionDelegate);
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("AUTH: Failed to get Steam auth ticket for web api"));
		CompletionDelegate.ExecuteIfBound(k_HAuthTicketInvalid, TEXT(""));
	}
}

FOnlineAuthSteam::SharedAuthUserSteamPtr FOnlineAuthSteam::GetUser(const FUniqueNetId& InUserId)
{
	if (SharedAuthUserSteamPtr* AuthUserPtr = AuthUsers.Find(InUserId.AsShared()))
	{
		return *AuthUserPtr;
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("AUTH: Trying to fetch user %s entry but the user does not exist"), *InUserId.ToString());
		return nullptr;
	}
}

FOnlineAuthSteam::SharedAuthUserSteamPtr FOnlineAuthSteam::GetOrCreateUser(const FUniqueNetId& InUserId)
{
	if (SharedAuthUserSteamPtr* AuthUserPtr = AuthUsers.Find(InUserId.AsShared()))
	{
		return *AuthUserPtr;
	}
	
	SharedAuthUserSteamPtr AuthUserPtr = MakeShareable(new FSteamAuthUser);
	AuthUsers.Add(InUserId.AsShared(), AuthUserPtr);
	return AuthUserPtr;
}

bool FOnlineAuthSteam::AuthenticateUser(const FUniqueNetId& InUserId)
{
	const FUniqueNetIdSteam& SteamUserId = FUniqueNetIdSteam::Cast(InUserId);
	if (SteamUserId.IsValid() && bEnabled)
	{
		// Create the user in the list if we don't already have them.
		SharedAuthUserSteamPtr TargetUser = GetOrCreateUser(SteamUserId);

		// Do not attempt to reauth this user if we are currently doing this.
		if (EnumHasAnyFlags(TargetUser->Status, ESteamAuthStatus::HasOrIsPendingAuth))
		{
			UE_LOG_ONLINE(Log, TEXT("AUTH: The user %s has authenticated or is currently authenticating. Skipping reauth"), *InUserId.ToString());
			return true;
		}

		// If the user has already failed auth, do not attempt to re-auth them.
		if (EnumHasAnyFlags(TargetUser->Status, ESteamAuthStatus::FailKick))
		{
			return false;
		}

		// Blank tickets are an immediate failure. A ticket should always have data.
		if (TargetUser->RecvTicket.IsEmpty())
		{
			UE_LOG_ONLINE(Warning, TEXT("AUTH: Ticket from user %s is empty"), *InUserId.ToString());
			TargetUser->Status |= ESteamAuthStatus::AuthFail;
			return false;
		}

		// If the ticket is over the size we're expecting, mark them as failure
		if (TargetUser->RecvTicket.Len() > STEAM_AUTH_MAX_TICKET_LENGTH_IN_BYTES)
		{
			UE_LOG_ONLINE(Warning, TEXT("AUTH: Ticket from user is over max size of ticket length"));
			TargetUser->Status |= ESteamAuthStatus::AuthFail;
			return false;
		}

		// Check to see if the ticket is actually in hex
		for (int32 i = 0; i < TargetUser->RecvTicket.Len(); ++i)
		{
			if (!CheckTCharIsHex(TargetUser->RecvTicket.GetCharArray()[i]))
			{
				UE_LOG_ONLINE(Warning, TEXT("AUTH: Ticket from user is not stored in hex!"));
				TargetUser->Status |= ESteamAuthStatus::AuthFail;
				return false;
			}
		}

		uint8 AuthTokenRaw[STEAM_AUTH_MAX_TICKET_LENGTH_IN_BYTES];
		int32 TicketSize = HexToBytes(TargetUser->RecvTicket, AuthTokenRaw);
		CSteamID UserCSteamId = SteamUserId;

		if (IsRunningDedicatedServer())
		{
			check(SteamServerPtr != nullptr);

			// For a dedicated server, we need to check the ticket's validity and boot if that check doesn't start properly.
			// Nothing else is needed on the ds.
			EBeginAuthSessionResult Result = SteamServerPtr->BeginAuthSession(AuthTokenRaw, TicketSize, UserCSteamId);
			if (Result == k_EBeginAuthSessionResultOK)
			{
				UE_LOG_ONLINE(Verbose, TEXT("AUTH: Steam user authentication task started for %s successfully"), *InUserId.ToString());
				TargetUser->Status |= ESteamAuthStatus::ValidationStarted;
				return true;
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("AUTH: User %s failed authentication %d"), *InUserId.ToString(), (int32)Result);
				TargetUser->Status |= ESteamAuthStatus::AuthFail;
			}
		}
		else
		{
			check(SteamUserPtr != nullptr);
			EBeginAuthSessionResult Result = SteamUserPtr->BeginAuthSession(AuthTokenRaw, TicketSize, UserCSteamId);
			if (Result == k_EBeginAuthSessionResultOK)
			{
				UE_LOG_ONLINE(Verbose, TEXT("AUTH: Steam user authentication task started for %s successfully"), *InUserId.ToString());
				TargetUser->Status |= ESteamAuthStatus::ValidationStarted;
				return true;
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("AUTH: User %s failed authentication %d"), *InUserId.ToString(), (int32)Result);
				TargetUser->Status |= ESteamAuthStatus::AuthFail;
			}
		}
	}
	else if(bEnabled)
	{
		UE_LOG_ONLINE(Warning, TEXT("AUTH: UserId was invalid!"));
	}
	return false;
}

void FOnlineAuthSteam::EndAuthentication(const FUniqueNetId& InUserId)
{
	const FUniqueNetIdSteam& SteamId = FUniqueNetIdSteam::Cast(InUserId);
	if (SteamId.IsValid())
	{
		CSteamID UserCSteamId = CSteamID(*(uint64*)SteamId.GetBytes());
		if (IsRunningDedicatedServer())
		{
			check(SteamServerPtr != nullptr);
			SteamServerPtr->EndAuthSession(UserCSteamId);
		}
		else
		{
			check(SteamUserPtr != nullptr);
			SteamUserPtr->EndAuthSession(UserCSteamId);
		}
		UE_LOG_ONLINE(Verbose, TEXT("AUTH: Ended authentication with %s"), *SteamId.ToString());
	}
}

void FOnlineAuthSteam::RevokeTicket(const uint32& Handle)
{
	if (SteamUserPtr != nullptr)
	{
		if (SteamTicketHandles.Contains(Handle))
		{
			SteamUserPtr->CancelAuthTicket(Handle);
			SteamTicketHandles.Remove(Handle);
			UE_LOG_ONLINE(Log, TEXT("AUTH: Revoking auth ticket with handle %d"), (int32)Handle);
		}
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("AUTH: Cannot revoke ticket with handle %d"), (int32)Handle);
	}
}

void FOnlineAuthSteam::RevokeAllTickets()
{
	UE_LOG_ONLINE(Log, TEXT("AUTH: Revoking all tickets."));

	// Removes all prior authorizations. Happens on disconnection.
	// Also cleans up any other previous auth data.
	for (SteamAuthentications::TIterator Users(AuthUsers); Users; ++Users)
	{
		EndAuthentication(*Users->Key);
	}

	// Clean up all handles if they haven't been cleared already
	if (SteamUserPtr != nullptr)
	{
		for (int HandleIdx = 0; HandleIdx < SteamTicketHandles.Num(); ++HandleIdx)
		{
			SteamUserPtr->CancelAuthTicket(SteamTicketHandles[HandleIdx]);
		}
	}

	SteamTicketHandles.Empty();
	AuthUsers.Empty();
}

void FOnlineAuthSteam::MarkPlayerForKick(const FUniqueNetId& InUserId)
{
	const FUniqueNetIdSteam& SteamId = FUniqueNetIdSteam::Cast(InUserId);
	SharedAuthUserSteamPtr TargetUser = GetUser(SteamId);
	if (TargetUser.IsValid())
	{
		TargetUser->Status |= ESteamAuthStatus::AuthFail;
		UE_LOG_ONLINE(Log, TEXT("AUTH: Marking %s for kick"), *InUserId.ToString());
	}
}

bool FOnlineAuthSteam::KickPlayer(const FUniqueNetId& InUserId, bool bSuppressFailure)
{
#if WITH_ENGINE
	bool bKickSuccess = false;
	const FUniqueNetIdSteam& SteamId = FUniqueNetIdSteam::Cast(InUserId);
	UWorld* World = (SteamSubsystem != nullptr) ? GetWorldForOnline(SteamSubsystem->GetInstanceName()) : nullptr;

	if (SteamUserPtr != nullptr && SteamUserPtr->GetSteamID() == SteamId)
	{
		if (!bSuppressFailure)
		{
			UE_LOG_ONLINE(Warning, TEXT("AUTH: Cannot kick ourselves!"));
		}
		return false;
	}

	// If we are overridden, respect that.
	if (AuthUtils.IsValid() && AuthUtils->OverrideFailureDelegate.IsBound())
	{
		AuthUtils->OverrideFailureDelegate.Execute(InUserId);
		RemoveUser(InUserId);
		return true;
	}

	if (World)
	{
		AGameModeBase* GameMode = World->GetAuthGameMode();
		if (GameMode == nullptr || GameMode->GameSession == nullptr)
		{
			if (!bSuppressFailure)
			{
				UE_LOG_ONLINE(Warning, TEXT("AUTH: Cannot kick player %s as we do not have a gamemode or session"), *InUserId.ToString());
			}
			return false;
		}

		for (FConstPlayerControllerIterator Itr = World->GetPlayerControllerIterator(); Itr; ++Itr)
		{
			APlayerController* PC = Itr->Get();
			if (PC && PC->PlayerState != nullptr && PC->PlayerState->GetUniqueId().IsValid() &&
				*(PC->PlayerState->GetUniqueId().GetUniqueNetId()) == InUserId)
			{
				const FText AuthKickReason = NSLOCTEXT("NetworkErrors", "HostClosedConnection", "Host closed the connection.");
				bKickSuccess = GameMode->GameSession->KickPlayer(PC, AuthKickReason);
				break;
			}
		}
	}

	// If we were able to kick them properly, call to remove their data.
	// Otherwise, they'll be attempted to be kicked again later.
	if (bKickSuccess)
	{
		UE_LOG_ONLINE(Log, TEXT("AUTH: Successfully kicked player %s"), *InUserId.ToString());
		RemoveUser(InUserId);
	}
	else if(!bSuppressFailure)
	{
		UE_LOG_ONLINE(Warning, TEXT("AUTH: Was not able to kick player %s Valid world: %d."), *InUserId.ToString(), (World != nullptr));
	}

	return bKickSuccess;
#else
	return false;
#endif //WITH_ENGINE
}

void FOnlineAuthSteam::RemoveUser(const FUniqueNetId& TargetUser)
{
	if (!IsServer() || !bEnabled)
	{
		return;
	}

	if (AuthUsers.Contains(TargetUser.AsShared()))
	{
		UE_LOG_ONLINE(Verbose, TEXT("AUTH: Removing user %s"), *TargetUser.ToString());
		EndAuthentication(TargetUser);
		AuthUsers.Remove(TargetUser.AsShared());
	}
}

bool FOnlineAuthSteam::Tick(float DeltaTime)
{
	if (!bEnabled || !IsServer())
	{
		return true;
	}

	// Loop through all users to detect if we need to do anything regarding resends
	for (SteamAuthentications::TIterator It(AuthUsers); It; ++It)
	{
		if (It->Value.IsValid())
		{
			SharedAuthUserSteamPtr CurUser = It->Value;
			const FUniqueNetId& CurUserId = *It->Key;

			// Kick any players that have failed authentication.
			if (EnumHasAnyFlags(CurUser->Status, ESteamAuthStatus::FailKick))
			{
				if (KickPlayer(CurUserId, EnumHasAnyFlags(CurUser->Status, ESteamAuthStatus::KickUser)))
				{
					// If we've modified the list, we can just end this frame.
					return true;
				}
				CurUser->Status |= ESteamAuthStatus::KickUser;
			}
		}
	}

	return true;
}

bool FOnlineAuthSteam::Exec(const TCHAR* Cmd)
{
#if UE_BUILD_SHIPPING
	return false;
#else
	bool bWasHandled = false;
	if (FParse::Command(&Cmd, TEXT("BADKEY")))
	{
		bWasHandled = true;
		bBadKey = !bBadKey;
		UE_LOG_ONLINE(Warning, TEXT("AUTH: Set send only bad auth keys flag to %d"), bBadKey);
	}
	else if (FParse::Command(&Cmd, TEXT("BADWRITES")))
	{
		bWasHandled = true;
		bBadWrite = !bBadWrite;
		UE_LOG_ONLINE(Warning, TEXT("AUTH: Set bad writes flag to %d"), bBadWrite);
	}
	else if (FParse::Command(&Cmd, TEXT("SENDBADID")))
	{
		bWasHandled = true;
		bSendBadId = !bSendBadId;
		UE_LOG_ONLINE(Warning, TEXT("AUTH: Set send bad id flag to %d"), bSendBadId);
	}
	else if (FParse::Command(&Cmd, TEXT("NEVERSENDKEY")))
	{
		bWasHandled = true;
		bNeverSendKey = !bNeverSendKey;
		UE_LOG_ONLINE(Warning, TEXT("AUTH: Set block key send flag to %d"), bNeverSendKey);
	}
	else if (FParse::Command(&Cmd, TEXT("REUSEKEY")))
	{
		bWasHandled = true;
		bReuseKey = !bReuseKey;
		UE_LOG_ONLINE(Warning, TEXT("AUTH: Set reuse auth key flag to %d"), bReuseKey);
	}
	else if (FParse::Command(&Cmd, TEXT("DROPALL")))
	{
		bWasHandled = true;
		bDropAll = !bDropAll;
		UE_LOG_ONLINE(Warning, TEXT("AUTH: Set drop all packets flag to %d"), bDropAll);
	}
	else if (FParse::Command(&Cmd, TEXT("DROPRANDOM")))
	{
		bWasHandled = true;
		bRandomDrop = !bRandomDrop;
		UE_LOG_ONLINE(Warning, TEXT("AUTH: Set drop random packets flag to %d"), bRandomDrop);
	}
	else if (FParse::Command(&Cmd, TEXT("ENABLE")))
	{
		bWasHandled = true;
		bEnabled = true;
		UE_LOG_ONLINE(Warning, TEXT("AUTH: Enabling the Auth Interface"));
	}
	else if (FParse::Command(&Cmd, TEXT("DISABLE")))
	{
		bWasHandled = true;
		bEnabled = false;
		UE_LOG_ONLINE(Warning, TEXT("AUTH: Disabling the Auth Interface"));
	}
	else if (FParse::Command(&Cmd, TEXT("FREEALLKEYS")))
	{
		bWasHandled = true;
		UE_LOG_ONLINE(Warning, TEXT("AUTH: Freeing all keys."));
		RevokeAllTickets();
	}
	else if (FParse::Command(&Cmd, TEXT("RESET")))
	{
		bEnabled = bWasHandled = true;
		bSendBadId = bNeverSendKey = bRandomDrop = bBadKey = bBadWrite = bDropAll = bReuseKey = false;
		UE_LOG_ONLINE(Warning, TEXT("AUTH: Reset all cheats."));
	}

	return bWasHandled;
#endif
}

void FOnlineAuthSteam::OnGetTicketForWebResponse(uint32 AuthTicketHandle, const FString& ResultToken)
{
	FOnGetAuthTicketForWebApiCompleteDelegate Delegate;
	if (ActiveAuthTicketForWebApiRequests.RemoveAndCopyValue(AuthTicketHandle, Delegate))
	{
		Delegate.ExecuteIfBound(AuthTicketHandle, ResultToken);
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("AUTH: Unexpected TicketForWebResponse."));
	}
}

void FOnlineAuthSteam::OnAuthResult(const FUniqueNetId& TargetId, int32 Response)
{
	if (!bEnabled)
	{
		return;
	}

	const FUniqueNetIdSteam& SteamId = FUniqueNetIdSteam::Cast(TargetId);
	if (!SteamId.IsValid())
	{
		UE_LOG_ONLINE(Warning, TEXT("AUTH: Auth Callback cannot process invalid users!"));
		return;
	}

	bool bDidAuthSucceed = (Response == k_EAuthSessionResponseOK);
	SharedAuthUserSteamPtr TargetUser = GetUser(TargetId);
	if (!TargetUser.IsValid())
	{
		// If we are missing an user here, this means that they were recently deleted or we never knew about them.
		UE_LOG_ONLINE(Warning, TEXT("AUTH: Could not find user data on result callback for %s, were they were recently deleted?"), 
			*SteamId.ToString());
		return;
	}

	// Remove the validation start flag
	TargetUser->Status &= ~ESteamAuthStatus::ValidationStarted;
	TargetUser->RecvTicket.Empty(); // Remove their ticket, we no longer need to store it.

	UE_LOG_ONLINE(Verbose, TEXT("AUTH: Finished auth with %s. Result ok? %d Response code %d"), *SteamId.ToString(), bDidAuthSucceed, Response);
	if (bDidAuthSucceed)
	{
		TargetUser->Status |= ESteamAuthStatus::AuthSuccess;
	}
	else
	{
		TargetUser->Status |= ESteamAuthStatus::AuthFail;
	}
	ExecuteResultDelegate(SteamId, bDidAuthSucceed, (ESteamAuthResponseCode)Response);
}

void FOnlineAuthSteam::ExecuteResultDelegate(const FUniqueNetId& TargetId, bool bWasSuccessful, ESteamAuthResponseCode ResponseCode)
{
	if (AuthUtils.IsValid())
	{
		// Fire both of these delegates.
		AuthUtils->OnAuthenticationResultDelegate.ExecuteIfBound(TargetId, bWasSuccessful);
		AuthUtils->OnAuthenticationResultWithCodeDelegate.ExecuteIfBound(TargetId, bWasSuccessful, ResponseCode);
	}
}

void FOnlineAuthSteam::FSteamAuthUser::SetKey(const FString& NewKey)
{
	if (!EnumHasAnyFlags(Status, ESteamAuthStatus::HasOrIsPendingAuth))
	{
		RecvTicket = NewKey;
	}
}

// Implementation of the public helper to determine if the Auth interface is enabled.
bool FOnlineAuthUtilsSteam::IsSteamAuthEnabled() const
{
	const FOnlineSubsystemSteam* SteamSub = static_cast<const FOnlineSubsystemSteam*>(IOnlineSubsystem::Get(STEAM_SUBSYSTEM));
	return (SteamSub && SteamSub->GetAuthInterface().IsValid() && SteamSub->GetAuthInterface()->IsSessionAuthEnabled());
}

