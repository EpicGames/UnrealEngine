// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemSteam.h"

enum class ESteamAuthResponseCode : uint8;

/** Steam Authentication Interface.
 *
 *  For the most part, this is fully automated. You simply just need to add the packet handler and your server will now
 *  require Steam Authentication for any incoming users. If a player fails to respond correctly, they will be kicked.
 *
 *  For projects that want to interface with SteamAuth and listen to callbacks, check out the OnlineAuthInterfaceUtils header.
 */

enum class ESteamAuthStatus : uint8
{
	None = 0,
	AuthSuccess = 1 << 0,
	AuthFail = 1 << 1,
	ValidationStarted = 1 << 2,
	KickUser = 1 << 3,
	FailKick = AuthFail | KickUser,
	HasOrIsPendingAuth = AuthSuccess | ValidationStarted
};

ENUM_CLASS_FLAGS(ESteamAuthStatus);

/**
 * Delegate executed when we complete a GetAuthTicketForWebApi call
 *
 * @param AuthTokenHandle The handle for the auth ticket
 * @param AuthTicket the auth ticket returned by the api
 */
DECLARE_DELEGATE_TwoParams(FOnGetAuthTicketForWebApiCompleteDelegate, uint32 /*AuthTokenHandle*/, const FString& /*AuthTicket*/);

class FOnlineAuthSteam
{
PACKAGE_SCOPE:
	FOnlineAuthSteam(FOnlineSubsystemSteam* InSubsystem, FOnlineAuthSteamUtilsPtr InAuthUtils);

	/** Data pertaining the current authentication state of the users in the game */
	struct FSteamAuthUser
	{
		FSteamAuthUser() : Status(ESteamAuthStatus::None) { }
		void SetKey(const FString& NewKey);
		// String representation of another user's ticket. Stored only temporarily
		FString RecvTicket;
		ESteamAuthStatus Status;
	};

	typedef TSharedPtr<FSteamAuthUser, ESPMode::NotThreadSafe> SharedAuthUserSteamPtr;

	SharedAuthUserSteamPtr GetUser(const FUniqueNetId& InUserId);
	SharedAuthUserSteamPtr GetOrCreateUser(const FUniqueNetId& InUserId);

	bool AuthenticateUser(const FUniqueNetId& InUserId);
	void EndAuthentication(const FUniqueNetId& InUserId);
	void MarkPlayerForKick(const FUniqueNetId& InUserId);
	void RevokeTicket(const uint32& Handle);
	void RevokeAllTickets();
	void RemoveUser(const FUniqueNetId& TargetUser);

	/** Generates Steam auth tickets */
	FString GetAuthTicket(uint32& AuthTokenHandle);
	void GetAuthTicketForWebApi(const FString& RemoteServiceIdentity, FOnGetAuthTicketForWebApiCompleteDelegate CompletionDelegate);

	bool Tick(float DeltaTime);
	bool Exec(const TCHAR* Cmd);

	/** Callbacks from Steam */
	void OnGetTicketForWebResponse(uint32 AuthTicketHandle, const FString& ResultToken);
	void OnAuthResult(const FUniqueNetId& TargetId, int32 Response);

	void ExecuteResultDelegate(const FUniqueNetId& TargetId, bool bWasSuccessful, ESteamAuthResponseCode ResponseCode);

private:
	typedef TUniqueNetIdMap<SharedAuthUserSteamPtr> SteamAuthentications;
	SteamAuthentications AuthUsers;
	TArray<uint32> SteamTicketHandles;
	TMap<uint32, FOnGetAuthTicketForWebApiCompleteDelegate> ActiveAuthTicketForWebApiRequests;

	/** Utility functions */
	FORCEINLINE bool IsServer() const
	{
		return SteamSubsystem != nullptr && SteamSubsystem->IsServer();
	}
	bool KickPlayer(const FUniqueNetId& InUserId, bool bSuppressFailure);

	FOnlineAuthSteam();

	/** Steam Interfaces */
	class ISteamUser* SteamUserPtr;
	class ISteamGameServer* SteamServerPtr;

	/** Cached pointer to owning subsystem */
	FOnlineSubsystemSteam* SteamSubsystem;
	FOnlineAuthSteamUtilsPtr AuthUtils;

	/** Settings */
	bool bEnabled;

	/** Testing flags */
PACKAGE_SCOPE:
	bool bBadKey;		// Send out invalid keys
	bool bReuseKey;		// Always send out the same key
	bool bBadWrite;		// Always make the bit writers have errors
	bool bDropAll;		// Drop all packets
	bool bRandomDrop;	// Randomly drop packets.
	bool bNeverSendKey;	// Client never sends their key.
	bool bSendBadId;	// Always send invalid steam ids.

	/** Setting Getters */
	bool IsSessionAuthEnabled() const { return bEnabled; }

public:
	virtual ~FOnlineAuthSteam();

	static uint32 GetMaxTicketSizeInBytes();
};
