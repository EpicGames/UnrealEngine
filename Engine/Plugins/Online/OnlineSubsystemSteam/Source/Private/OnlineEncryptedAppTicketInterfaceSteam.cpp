// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineEncryptedAppTicketInterfaceSteam.h"
#include "OnlineAsyncTaskManagerSteam.h"
#include "OnlineSubsystemSteam.h"
#include "SteamUtilities.h"

enum class ESteamEncryptedAppTicketState
{
	None = 0,

	TicketRequested,
	TicketAvailable,
	TicketFailure
};

/**
 *	Async task to retrieve encrypted application ticket data from Steam backend.
 */
class FOnlineAsyncTaskSteamRequestEncryptedAppTicket : public FOnlineAsyncTaskSteam
{
private:

	/** Has this task been initialized yet */
	bool bInitialized;
	/** Data to encrypt in the application ticket. */
	TArray<uint8> DataToEncrypt;
	/** Returned results from Steam */
	EncryptedAppTicketResponse_t CallbackResults;

public:

	FOnlineAsyncTaskSteamRequestEncryptedAppTicket(FOnlineSubsystemSteam* InSteamSubsystem) :
		FOnlineAsyncTaskSteam(InSteamSubsystem, k_uAPICallInvalid),
		bInitialized(false)
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncTaskSteamRequestEncryptedAppTicket bWasSuccessful: %d"), WasSuccessful());
	}

	/**
	 * Sets the optional data to include for the encrypted application ticket.
	 *
	 * @param Data	Input data to move to the internal buffer of the class.
	 */
	void SetData(TArray<uint8>&& Data)
	{
		DataToEncrypt = MoveTemp(Data);
	}

	/**
	 * Give the async task time to do its work.
	 * Can only be called on the async task manager thread.
	 */
	virtual void Tick() override
	{
		if (!bInitialized)
		{
			ISteamUser* const SteamUserPtr = SteamUser();
			if (SteamUserPtr != nullptr)
			{
				const int SizeOfData = DataToEncrypt.Num();
				void* const DataPtr = SizeOfData > 0 ? DataToEncrypt.GetData() : nullptr;
				CallbackHandle = SteamUserPtr->RequestEncryptedAppTicket(DataPtr, SizeOfData);
				bInitialized = true;
			}
		}

		if (CallbackHandle != k_uAPICallInvalid)
		{
			ISteamUtils* SteamUtilsPtr = SteamUtils();
			check(SteamUtilsPtr);

			// Poll for completion status
			bool bFailedCall = false;
			bIsComplete = SteamUtilsPtr->IsAPICallCompleted(CallbackHandle, &bFailedCall) ? true : false;
			if (bIsComplete)
			{
				bool bFailedResult;
				// Retrieve the callback data from the request
				bool bSuccessCallResult = SteamUtilsPtr->GetAPICallResult(CallbackHandle, &CallbackResults, sizeof(CallbackResults), CallbackResults.k_iCallback, &bFailedResult);
				bWasSuccessful = (bSuccessCallResult ? true : false) &&
					(!bFailedCall ? true : false) &&
					(!bFailedResult ? true : false);
			}
		}
		else
		{
			bWasSuccessful = false;
			bIsComplete = false;
		}
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread.
	 * Can only be called on the game thread by the async task manager.
	 */
	virtual void Finalize() override
	{
		FOnlineAsyncTaskSteam::Finalize();

		if (CallbackResults.m_eResult != k_EResultOK)
		{
			bWasSuccessful = false;
		}

		if (!bWasSuccessful)
		{
			UE_LOG_ONLINE(Warning, TEXT("Failed to obtain encrypted application ticket, result code: %s"),
						  *SteamResultString(CallbackResults.m_eResult));
		}
	}

	/**
	 *	Async task is given a chance to trigger its delegates.
	 */
	virtual void TriggerDelegates() override
	{
		FOnlineAsyncTaskSteam::TriggerDelegates();

		FOnlineEncryptedAppTicketSteamPtr Interface = Subsystem->GetEncryptedAppTicketInterface();
		if (Interface.IsValid())
		{
			Interface->OnAPICallComplete(bWasSuccessful, CallbackResults.m_eResult);
		}
	}
};

FOnlineEncryptedAppTicketSteam::FOnlineEncryptedAppTicketSteam(FOnlineSubsystemSteam* InSubsystem) :
	FSelfRegisteringExec(),
	SteamSubsystem(InSubsystem),
	TicketState(ESteamEncryptedAppTicketState::None)
{
}

FOnlineEncryptedAppTicketSteam::~FOnlineEncryptedAppTicketSteam()
{
	OnEncryptedAppTicketResultDelegate.Clear();
}

bool FOnlineEncryptedAppTicketSteam::RequestEncryptedAppTicket(void* DataToEncrypt, int SizeOfDataToEncrypt)
{
	// Return instantly if encrypted application ticket is already being waited for but no data has yet been received.
	if (TicketState == ESteamEncryptedAppTicketState::TicketRequested)
	{
		UE_LOG_ONLINE(Warning, TEXT("Request dropped, prior encrypted application ticket request being processed."));
		return false;
	}

	TicketState = ESteamEncryptedAppTicketState::TicketRequested;

	FOnlineAsyncTaskSteamRequestEncryptedAppTicket* NewTask = new FOnlineAsyncTaskSteamRequestEncryptedAppTicket(SteamSubsystem);
	if (DataToEncrypt && SizeOfDataToEncrypt > 0)
	{
		TArray<uint8> Data((uint8*)DataToEncrypt, SizeOfDataToEncrypt);
		NewTask->SetData(MoveTemp(Data));
	}

	SteamSubsystem->QueueAsyncTask(NewTask);
	return true;
}

void FOnlineEncryptedAppTicketSteam::OnAPICallComplete(bool bEncryptedDataAvailable, int32 ResultCode)
{
	// Update object state based on success of the Steam API call.
	TicketState = bEncryptedDataAvailable ? ESteamEncryptedAppTicketState::TicketAvailable :
											ESteamEncryptedAppTicketState::TicketFailure;

	// Inform any listeners about the Steam API call result.
	OnEncryptedAppTicketResultDelegate.Broadcast(bEncryptedDataAvailable, ResultCode);
}

bool FOnlineEncryptedAppTicketSteam::GetEncryptedAppTicket(TArray<uint8>& OutEncryptedData)
{
	if (TicketState == ESteamEncryptedAppTicketState::TicketAvailable)
	{
		// Note: Ticket data is retrievable till another encrypted application ticket request replaces it.
		ISteamUser* const SteamUserPtr = SteamUser();
		if (SteamUserPtr)
		{
			uint32 ExactTicketSize = 0;
			SteamUserPtr->GetEncryptedAppTicket(nullptr, 0, &ExactTicketSize);

			if (ExactTicketSize > 0)
			{
				OutEncryptedData.Reset();
				OutEncryptedData.Reserve(ExactTicketSize);
				OutEncryptedData.AddUninitialized(ExactTicketSize);

				if (SteamUserPtr->GetEncryptedAppTicket(OutEncryptedData.GetData(),
														OutEncryptedData.GetAllocatedSize(),
														&ExactTicketSize))
				{
					return true;
				}
			}

			UE_LOG_ONLINE(Warning, TEXT("Getting encrypted application ticket failed!"));
		}
	}
	else
	{
		switch (TicketState)
		{
			case ESteamEncryptedAppTicketState::None:
				UE_LOG_ONLINE(Warning, TEXT("Unable to get encrypted application ticket, it hasn't been requested!"));
				break;

			case ESteamEncryptedAppTicketState::TicketRequested:
				UE_LOG_ONLINE(Warning, TEXT("Encrypted ticket is not yet available!"));
				break;

			case ESteamEncryptedAppTicketState::TicketFailure:
				UE_LOG_ONLINE(Warning, TEXT("Failed to get encrypted application ticket due to the original request failing."));
				break;

			default:
				UE_LOG_ONLINE(Warning, TEXT("Failed to get encrypted application ticket, unknown reason."));
				break;
		}
	}

	// Remove any existing data from the outgoing container before returning failure.
	OutEncryptedData.Reset();
	return false;
}

bool FOnlineEncryptedAppTicketSteam::Exec_Dev(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
#if !UE_BUILD_SHIPPING
	if (FParse::Command(&Cmd, TEXT("RequestEncryptedAppTicket")))
	{
		// Usage examples:
		// RequestEncryptedAppTicket
		// RequestEncryptedAppTicket "Additional data to encrypt"
		FString StringDataToEncrypt = FParse::Token(Cmd, 0);

		int SizeOfDataToEncrypt = 0;
		void* DataToEncryptPtr = nullptr;
		if (StringDataToEncrypt.GetAllocatedSize() > 0)
		{
			DataToEncryptPtr = &StringDataToEncrypt[0];
			SizeOfDataToEncrypt = StringDataToEncrypt.Len();
		}

		Ar.Log(ELogVerbosity::Display, FString::Printf(
			   TEXT("Requesting encrypted application ticket: DataToEncrypt: %p, SizeOfDataToEncrypt: %d."),
			   DataToEncryptPtr, SizeOfDataToEncrypt));

		const bool bSuccess = RequestEncryptedAppTicket(DataToEncryptPtr, SizeOfDataToEncrypt);

		Ar.Log(ELogVerbosity::Display,
			   FString::Printf(TEXT("Requesting encrypted application ticket %s"),
			   bSuccess ? TEXT("SUCCEEDED.") : TEXT("FAILED!")));

		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("GetEncryptedAppTicket")))
	{
		// Usage examples:
		// GetEncryptedAppTicket
		TArray<uint8> DataBuffer;
		Ar.Log(ELogVerbosity::Display, TEXT("Trying to get encrypted application ticket."));

		const bool bSuccess = GetEncryptedAppTicket(DataBuffer);
		if (bSuccess)
		{
			FString RetrievedHexBytes = BytesToHex(DataBuffer.GetData(), DataBuffer.Num());
			Ar.Log(ELogVerbosity::Display, FString::Printf(
				   TEXT("Getting encrypted application ticket SUCCEEDED, encrypted ticket size: %u, data retrieved: %s"),
				   DataBuffer.Num(), *RetrievedHexBytes));
		}
		else
		{
			Ar.Log(ELogVerbosity::Display, TEXT("Getting encrypted application ticket FAILED!"));
		}

		return true;
	}
#endif

	return false;
}
