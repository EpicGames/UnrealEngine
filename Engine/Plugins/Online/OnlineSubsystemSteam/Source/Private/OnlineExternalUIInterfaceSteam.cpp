// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineExternalUIInterfaceSteam.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSessionSettings.h"
#include "OnlineSubsystemSteam.h"
#include "OnlineSubsystemSteamTypes.h"

// Other external UI possibilities in Steam
// "Players" - recently played with players
// "Community" 
// "Settings"
// "OfficialGameGroup"
// "Stats"

FString FOnlineAsyncEventSteamExternalUITriggered::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncEventSteamExternalUITriggered bIsActive: %d"), bIsActive);
}

void FOnlineAsyncEventSteamExternalUITriggered::TriggerDelegates()
{
	FOnlineAsyncEvent::TriggerDelegates();
	IOnlineExternalUIPtr ExternalUIInterface = Subsystem->GetExternalUIInterface();
	ExternalUIInterface->TriggerOnExternalUIChangeDelegates(bIsActive);

	// Calling this to mimic behavior as close as possible with other platforms where this delegate is passed in (such as PS4/Xbox).
	if (!bIsActive)
	{
		FOnlineExternalUISteamPtr ExternalUISteam = StaticCastSharedPtr<FOnlineExternalUISteam>(ExternalUIInterface);
		ExternalUISteam->ProfileUIClosedDelegate.ExecuteIfBound();
		ExternalUISteam->ProfileUIClosedDelegate.Unbind();

		// We don't have a way to tell if you sent a message, but we attempt to send it for you.
		ExternalUISteam->ShowMessageClosedDelegate.ExecuteIfBound(ExternalUISteam->bMessageSent);
		ExternalUISteam->ShowMessageClosedDelegate.Unbind();
		ExternalUISteam->bMessageSent = false;

		// We don't have any way to know that you bought an item on the store from this overlay. 
		// This would be handled either by a DLC query or the server WebAPI.
		// This returns true in order to trigger license checks.
		ExternalUISteam->ShowStoreClosedDelegate.ExecuteIfBound(true);
		ExternalUISteam->ShowStoreClosedDelegate.Unbind();

		// Steam doesn't allow you to capture the final browsing url on web overlays, so pass an empty string
		ExternalUISteam->ShowWebUrlClosedDelegate.ExecuteIfBound(TEXT(""));
		ExternalUISteam->ShowWebUrlClosedDelegate.Unbind();
	}
}

bool FOnlineExternalUISteam::ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate)
{
	return false;
}

bool FOnlineExternalUISteam::ShowAccountCreationUI(const int ControllerIndex, const FOnAccountCreationUIClosedDelegate& Delegate)
{
	return false;
}

bool FOnlineExternalUISteam::ShowFriendsUI(int32 LocalUserNum)
{
	SteamFriends()->ActivateGameOverlay("Friends");
	return true;
}

bool FOnlineExternalUISteam::ShowInviteUI(int32 LocalUserNum, FName SessionName)
{
	IOnlineSessionPtr SessionInt = SteamSubsystem->GetSessionInterface();
	if (!SessionInt.IsValid())
	{
		return false;
	}

	const FNamedOnlineSession* const Session = SessionInt->GetNamedSession(SessionName);
	if (Session && Session->SessionInfo.IsValid())
	{
		const FOnlineSessionInfoSteam* const SessionInfo = (FOnlineSessionInfoSteam*)(Session->SessionInfo.Get());
		if (SessionInfo->SessionType == ESteamSession::LobbySession && SessionInfo->SessionId->IsValid())
		{
			// This can only invite to lobbies, does not work for dedicated servers.
			SteamFriends()->ActivateGameOverlayInviteDialog(*SessionInfo->SessionId);
		}
		else if(SessionInfo->SessionType == ESteamSession::AdvertisedSessionHost || SessionInfo->SessionType == ESteamSession::AdvertisedSessionClient)
		{
			// Invite people to start this game.
			// To invite someone directly into the game, use SendSessionInviteToFriend
			SteamFriends()->ActivateGameOverlay("LobbyInvite");
		}
		return true;
	}

	return false;
}

bool FOnlineExternalUISteam::ShowAchievementsUI(int32 LocalUserNum)
{
	SteamFriends()->ActivateGameOverlay("Achievements");
	return true;
}

bool FOnlineExternalUISteam::ShowLeaderboardUI(const FString& LeaderboardName)
{
	return false;
}

bool FOnlineExternalUISteam::ShowWebURL(const FString& Url, const FShowWebUrlParams& ShowParams, const FOnShowWebUrlClosedDelegate& Delegate)
{
	if (!Url.StartsWith(TEXT("https://")))
	{
		SteamFriends()->ActivateGameOverlayToWebPage(TCHAR_TO_UTF8(*FString::Printf(TEXT("https://%s"), *Url)));
	}
	else
	{
		SteamFriends()->ActivateGameOverlayToWebPage(TCHAR_TO_UTF8(*Url));
	}

	ShowWebUrlClosedDelegate = Delegate;
	return true;
}

bool FOnlineExternalUISteam::CloseWebURL()
{
	return false;
}

bool FOnlineExternalUISteam::ShowProfileUI(const FUniqueNetId& Requestor, const FUniqueNetId& Requestee, const FOnProfileUIClosedDelegate& Delegate)
{
	SteamFriends()->ActivateGameOverlayToUser(TCHAR_TO_UTF8(TEXT("steamid")), (const FUniqueNetIdSteam&)Requestee);

	ProfileUIClosedDelegate = Delegate;
	return true;
}

bool FOnlineExternalUISteam::ShowAccountUpgradeUI(const FUniqueNetId& UniqueId)
{
	return false;
}

bool FOnlineExternalUISteam::ShowStoreUI(int32 LocalUserNum, const FShowStoreParams& ShowParams, const FOnShowStoreUIClosedDelegate& Delegate)
{
	if (!ShowParams.ProductId.IsNumeric() || ShowParams.ProductId.IsEmpty())
	{
		return false;
	}

	uint32 ProductId = (uint32)FCString::Atoi(*ShowParams.ProductId);

	if (ProductId == 0)
	{
		return false;
	}

	SteamFriends()->ActivateGameOverlayToStore(ProductId, ShowParams.bAddToCart ? k_EOverlayToStoreFlag_AddToCartAndShow : k_EOverlayToStoreFlag_None);
	ShowStoreClosedDelegate = Delegate;

	return true;
}

bool FOnlineExternalUISteam::ShowSendMessageUI(int32 LocalUserNum, const FShowSendMessageParams& ShowParams, const FOnShowSendMessageUIClosedDelegate& Delegate)
{
	// Steam only allows an application to open the chat UI if a recipient is specified.
	return false;
}

bool FOnlineExternalUISteam::ShowSendMessageToUserUI(int32 LocalUserNum, const FUniqueNetId& Recipient, const FShowSendMessageParams& ShowParams, const FOnShowSendMessageUIClosedDelegate& Delegate)
{
	const FUniqueNetIdSteam& TargetUser = (const FUniqueNetIdSteam&)Recipient;
	const FString MessageToSend = ShowParams.DisplayMessage.ToString();

	if (!TargetUser.IsValid() || MessageToSend.IsEmpty())
	{
		return false;
	}
	ShowMessageClosedDelegate = Delegate;

	bMessageSent = SteamFriends()->ReplyToFriendMessage(TargetUser, TCHAR_TO_UTF8(*MessageToSend));
	SteamFriends()->ActivateGameOverlayToUser(TCHAR_TO_UTF8(TEXT("chat")), TargetUser);
	
	return true;
}
