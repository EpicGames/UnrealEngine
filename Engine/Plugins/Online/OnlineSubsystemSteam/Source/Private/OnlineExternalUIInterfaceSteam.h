// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManager.h"
#include "Interfaces/OnlineExternalUIInterface.h"
#include "OnlineSubsystemSteamPackage.h"

class FOnlineSubsystemSteam;

/**
 *	Async event that notifies when the STEAM external UI has been activated
 */
class FOnlineAsyncEventSteamExternalUITriggered : public FOnlineAsyncEvent<FOnlineSubsystemSteam>
{
	/** Is the External UI activating */
	bool bIsActive;

	/** Hidden on purpose */
	FOnlineAsyncEventSteamExternalUITriggered() :
		FOnlineAsyncEvent(NULL),
		bIsActive(false)
	{
	}

public:

	FOnlineAsyncEventSteamExternalUITriggered(FOnlineSubsystemSteam* InSteamSubsystem, bool bInIsActive) :
		FOnlineAsyncEvent(InSteamSubsystem),
		bIsActive(bInIsActive)
	{
	}

	virtual ~FOnlineAsyncEventSteamExternalUITriggered()
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override;

	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override;
};

/** 
 * Implementation for the STEAM external UIs
 */
class FOnlineExternalUISteam : public IOnlineExternalUI
{

private:
	/** Reference to the main Steam subsystem */
	class FOnlineSubsystemSteam* SteamSubsystem;

	/** Hidden on purpose */
	FOnlineExternalUISteam() = delete;

PACKAGE_SCOPE:

	FOnlineExternalUISteam(class FOnlineSubsystemSteam* InSteamSubsystem) :
		SteamSubsystem(InSteamSubsystem),
		bMessageSent(false)
	{
	}

	/** If a message was successfully sent via the Steam UI */
	bool bMessageSent;

	/** Triggered when the steam overlay is closed if it was opened via ShowProfileUI. Delegate will be unbound after it is executed. */
	FOnProfileUIClosedDelegate ProfileUIClosedDelegate;
	FOnShowWebUrlClosedDelegate ShowWebUrlClosedDelegate;
	FOnShowSendMessageUIClosedDelegate ShowMessageClosedDelegate;
	FOnShowStoreUIClosedDelegate ShowStoreClosedDelegate;

public:

	virtual ~FOnlineExternalUISteam()
	{
	}

	// IOnlineExternalUI
	virtual bool ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate = FOnLoginUIClosedDelegate()) override;
	virtual bool ShowAccountCreationUI(const int ControllerIndex, const FOnAccountCreationUIClosedDelegate& Delegate = FOnAccountCreationUIClosedDelegate()) override;
	virtual bool ShowFriendsUI(int32 LocalUserNum) override;
	virtual bool ShowInviteUI(int32 LocalUserNum, FName SessionName = NAME_GameSession) override;
	virtual bool ShowAchievementsUI(int32 LocalUserNum) override;
	virtual bool ShowLeaderboardUI(const FString& LeaderboardName) override;
	virtual bool ShowWebURL(const FString& Url, const FShowWebUrlParams& ShowParams, const FOnShowWebUrlClosedDelegate& Delegate = FOnShowWebUrlClosedDelegate()) override;
	virtual bool CloseWebURL() override;
	virtual bool ShowProfileUI(const FUniqueNetId& Requestor, const FUniqueNetId& Requestee, const FOnProfileUIClosedDelegate& Delegate = FOnProfileUIClosedDelegate()) override;
	virtual bool ShowAccountUpgradeUI(const FUniqueNetId& UniqueId) override;
	virtual bool ShowStoreUI(int32 LocalUserNum, const FShowStoreParams& ShowParams, const FOnShowStoreUIClosedDelegate& Delegate = FOnShowStoreUIClosedDelegate()) override;
	virtual bool ShowSendMessageUI(int32 LocalUserNum, const FShowSendMessageParams& ShowParams, const FOnShowSendMessageUIClosedDelegate& Delegate = FOnShowSendMessageUIClosedDelegate()) override;
	virtual bool ShowSendMessageToUserUI(int32 LocalUserNum, const FUniqueNetId& Recipient, const FShowSendMessageParams& ShowParams, const FOnShowSendMessageUIClosedDelegate& Delegate = FOnShowSendMessageUIClosedDelegate()) override;
};

typedef TSharedPtr<FOnlineExternalUISteam, ESPMode::ThreadSafe> FOnlineExternalUISteamPtr;

