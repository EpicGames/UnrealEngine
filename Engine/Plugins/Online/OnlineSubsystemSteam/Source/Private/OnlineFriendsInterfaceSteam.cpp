// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineFriendsInterfaceSteam.h"
#include "OnlinePresenceInterfaceSteam.h"
#include "OnlineSubsystemSteam.h"
#include "OnlineError.h"
#include "OnlineSubsystemSteamTypes.h"
#include <steam/isteamuser.h>

// FOnlineFriendSteam
const FOnlineUserPresence FOnlineFriendSteam::EmptyPresence;

FOnlineFriendSteam::FOnlineFriendSteam(const CSteamID& InUserId)
	: UserId(FUniqueNetIdSteam::Create(InUserId))
{
}

FUniqueNetIdRef FOnlineFriendSteam::GetUserId() const
{
	return UserId;
}

FString FOnlineFriendSteam::GetRealName() const
{
	FString Result;
	GetAccountData(TEXT("nickname"),Result);
	return Result;
}

FString FOnlineFriendSteam::GetDisplayName(const FString& Platform) const
{
	FString Result;
	GetAccountData(TEXT("nickname"),Result);
	return Result;
}

bool FOnlineFriendSteam::GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	return GetAccountData(AttrName,OutAttrValue);
}

EInviteStatus::Type FOnlineFriendSteam::GetInviteStatus() const
{
	return EInviteStatus::Accepted;
}

const FOnlineUserPresence& FOnlineFriendSteam::GetPresence() const
{
	// Get the steam subsystem
	if (IOnlineSubsystem* SteamOSS = IOnlineSubsystem::Get(STEAM_SUBSYSTEM))
	{
		// Get the steam presence interface
		if (FOnlinePresenceSteam* OnlinePresence = static_cast<FOnlinePresenceSteam*>(SteamOSS->GetPresenceInterface().Get()))
		{
			// Get the presence if it's available
			TSharedPtr<FOnlineUserPresence> Presence;
			EOnlineCachedResult::Type Result = OnlinePresence->GetCachedPresence(*UserId, Presence);

			// If the user's presence was not cached (it might be the first time)
			if (Result == EOnlineCachedResult::Type::NotFound)
			{
				// Add the user's presence to the cache
				OnlinePresence->UpdatePresenceForUser(*UserId);

				// Try getting the user's presence again
				OnlinePresence->GetCachedPresence(*UserId, Presence);
			}

			if(Presence.IsValid())
			{
				return *Presence;
			}
		}
	}

	return EmptyPresence;
}

// FOnlineFriendsStream

FOnlineFriendsSteam::FOnlineFriendsSteam(FOnlineSubsystemSteam* InSteamSubsystem) :
	SteamSubsystem(InSteamSubsystem),
	SteamUserPtr(NULL),
	SteamFriendsPtr(NULL)
{
	check(SteamSubsystem);
	SteamUserPtr = SteamUser();
	SteamFriendsPtr = SteamFriends();
}

bool FOnlineFriendsSteam::ReadFriendsList(int32 LocalUserNum, const FString& ListName, const FOnReadFriendsListComplete& Delegate /*= FOnReadFriendsListComplete()*/)
{
	FString ErrorStr;
	if (LocalUserNum < MAX_LOCAL_PLAYERS &&
		SteamUserPtr != NULL &&
		SteamUserPtr->BLoggedOn() &&
		SteamFriendsPtr != NULL)
	{
		SteamSubsystem->QueueAsyncTask(new FOnlineAsyncTaskSteamReadFriendsList(this, LocalUserNum, ListName, Delegate));
	}
	else
	{
		ErrorStr = FString::Printf(TEXT("No valid LocalUserNum=%d"), LocalUserNum);
	}
	if (!ErrorStr.IsEmpty())
	{
		Delegate.ExecuteIfBound(LocalUserNum, false, ListName, ErrorStr);
		return false;
	}
	return true;
}

bool FOnlineFriendsSteam::DeleteFriendsList(int32 LocalUserNum, const FString& ListName, const FOnDeleteFriendsListComplete& Delegate /*= FOnDeleteFriendsListComplete()*/)
{
	Delegate.ExecuteIfBound(LocalUserNum, false, ListName, FString(TEXT("DeleteFriendsList() is not supported")));
	return false;
}

bool FOnlineFriendsSteam::SendInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnSendInviteComplete& Delegate /*= FOnSendInviteComplete()*/)
{
	Delegate.ExecuteIfBound(LocalUserNum, false, FriendId, ListName, FString(TEXT("SendInvite() is not supported")));
	return false;
}

bool FOnlineFriendsSteam::AcceptInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnAcceptInviteComplete& Delegate /*= FOnAcceptInviteComplete()*/)
{
	Delegate.ExecuteIfBound(LocalUserNum, false, FriendId, ListName, FString(TEXT("AcceptInvite() is not supported")));
	return false;
}

bool FOnlineFriendsSteam::RejectInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName)
{
	TriggerOnRejectInviteCompleteDelegates(LocalUserNum, false, FriendId, ListName, FString(TEXT("RejectInvite() is not supported")));
	return false;
}

void FOnlineFriendsSteam::SetFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FString& Alias, const FOnSetFriendAliasComplete& Delegate /*= FOnSetFriendAliasComplete()*/)
{
	FUniqueNetIdRef FriendIdRef = FriendId.AsShared();
	SteamSubsystem->ExecuteNextTick([LocalUserNum, FriendIdRef, ListName, Delegate]()
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsSteam::SetFriendAlias is not supported"));
		Delegate.ExecuteIfBound(LocalUserNum, *FriendIdRef, ListName, FOnlineError(EOnlineErrorResult::NotImplemented));
	});
}

void FOnlineFriendsSteam::DeleteFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnDeleteFriendAliasComplete& Delegate)
{
	FUniqueNetIdRef FriendIdRef = FriendId.AsShared();
	SteamSubsystem->ExecuteNextTick([LocalUserNum, FriendIdRef, ListName, Delegate]()
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsSteam::DeleteFriendAlias is not supported"));
		Delegate.ExecuteIfBound(LocalUserNum, *FriendIdRef, ListName, FOnlineError(EOnlineErrorResult::NotImplemented));
	});
}

bool FOnlineFriendsSteam::DeleteFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName)
{
	TriggerOnDeleteFriendCompleteDelegates(LocalUserNum, false, FriendId, ListName, FString(TEXT("DeleteFriend() is not supported")));
	return false;
}

bool FOnlineFriendsSteam::GetFriendsList(int32 LocalUserNum, const FString& ListName, TArray< TSharedRef<FOnlineFriend> >& OutFriends)
{
	bool bResult = false;
	if (LocalUserNum < MAX_LOCAL_PLAYERS &&
		SteamUserPtr != NULL &&
		SteamUserPtr->BLoggedOn() &&
		SteamFriendsPtr != NULL)
	{
		FSteamFriendsList* FriendsList = FriendsLists.Find(LocalUserNum);
		if (FriendsList != NULL)
		{
			for (int32 FriendIdx=0; FriendIdx < FriendsList->Friends.Num(); FriendIdx++)
			{
				OutFriends.Add(FriendsList->Friends[FriendIdx]);
			}
			bResult = true;
		}
	}
	return bResult;
}

TSharedPtr<FOnlineFriend> FOnlineFriendsSteam::GetFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName)
{
	TSharedPtr<FOnlineFriend> Result;
	if (LocalUserNum < MAX_LOCAL_PLAYERS &&
		SteamUserPtr != NULL &&
		SteamUserPtr->BLoggedOn() &&
		SteamFriendsPtr != NULL)
	{
		FSteamFriendsList* FriendsList = FriendsLists.Find(LocalUserNum);
		if (FriendsList != NULL)
		{
			for (int32 FriendIdx=0; FriendIdx < FriendsList->Friends.Num(); FriendIdx++)
			{
				if (*FriendsList->Friends[FriendIdx]->GetUserId() == FriendId)
				{
					Result = FriendsList->Friends[FriendIdx];
					break;
				}
			}
		}
	}
	return Result;
}

bool FOnlineFriendsSteam::IsFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName)
{
	bool bIsFriend = false;
	if (LocalUserNum < MAX_LOCAL_PLAYERS &&
		SteamUserPtr != NULL &&
		SteamUserPtr->BLoggedOn() &&
		SteamFriendsPtr != NULL)
	{
		// Ask Steam if they are on the buddy list
		const CSteamID SteamPlayerId(*(uint64*)FriendId.GetBytes());
		bIsFriend = SteamFriendsPtr->GetFriendRelationship(SteamPlayerId) == k_EFriendRelationshipFriend;
	}
	return bIsFriend;
}

bool FOnlineFriendsSteam::QueryRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace)
{
	UE_LOG_ONLINE_FRIEND(Verbose, TEXT("FOnlineFriendsSteam::QueryRecentPlayers()"));

	TriggerOnQueryRecentPlayersCompleteDelegates(UserId, Namespace, false, TEXT("not implemented"));

	return false;
}

bool FOnlineFriendsSteam::GetRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace, TArray< TSharedRef<FOnlineRecentPlayer> >& OutRecentPlayers)
{
	return false;
}

void FOnlineFriendsSteam::DumpRecentPlayers() const
{
}

bool FOnlineFriendsSteam::BlockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId)
{
	return false;
}

bool FOnlineFriendsSteam::UnblockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId)
{
	return false;
}

bool FOnlineFriendsSteam::QueryBlockedPlayers(const FUniqueNetId& UserId)
{
	return false;
}

bool FOnlineFriendsSteam::GetBlockedPlayers(const FUniqueNetId& UserId, TArray< TSharedRef<FOnlineBlockedPlayer> >& OutBlockedPlayers)
{
	return false;
}

void FOnlineFriendsSteam::DumpBlockedPlayers() const
{
}

bool FOnlineAsyncTaskSteamReadFriendsList::CanAddUserToList(bool bIsOnline, bool bIsPlayingThisGame, bool bIsPlayingGameInSession)
{
	switch (FriendsListFilter)
	{
		default:
		case EFriendsLists::Default:
			return true;
		case EFriendsLists::OnlinePlayers:
			return bIsOnline;
		case EFriendsLists::InGamePlayers:
			return bIsOnline && bIsPlayingThisGame;
		case EFriendsLists::InGameAndSessionPlayers:
			return bIsOnline && bIsPlayingThisGame && bIsPlayingGameInSession;
	}
}

void FOnlineAsyncTaskSteamReadFriendsList::Finalize()
{
	FOnlineSubsystemSteam* SteamSubsystem = FriendsPtr->SteamSubsystem;
	ISteamFriends* SteamFriendsPtr = FriendsPtr->SteamFriendsPtr;
	FOnlineFriendsSteam::FSteamFriendsList& FriendsList = FriendsPtr->FriendsLists.FindOrAdd(LocalUserNum);

	const int32 NumFriends = SteamFriendsPtr->GetFriendCount(k_EFriendFlagImmediate);
	// Pre-size the array for minimal re-allocs
	FriendsList.Friends.Empty(NumFriends);
	// Loop through all the friends adding them one at a time
	for (int32 Index = 0; Index < NumFriends; Index++)
	{
		const CSteamID SteamPlayerId = SteamFriendsPtr->GetFriendByIndex(Index, k_EFriendFlagImmediate);
		const FString NickName(UTF8_TO_TCHAR(SteamFriendsPtr->GetFriendPersonaName(SteamPlayerId)));

		// Get this user's friend information so we can figure out if we can add them to our list.
		bool bInASession;
		FriendGameInfo_t FriendGameInfo;
		bool bIsPlayingAGame = SteamFriendsPtr->GetFriendGamePlayed(SteamPlayerId, &FriendGameInfo);
		bool bIsOnline = (SteamFriendsPtr->GetFriendPersonaState(SteamPlayerId) >= k_EPersonaStateOnline);
		bool bIsPlayingThisGame = (FriendGameInfo.m_gameID.AppID() == SteamSubsystem->GetSteamAppId());
		bool bHasConnectInformation = (SteamFriendsPtr->GetFriendRichPresence(SteamPlayerId, "connect") != nullptr);
		FString JoinablePresenceString = UTF8_TO_TCHAR(SteamFriendsPtr->GetFriendRichPresence(SteamPlayerId, "Joinable"));

		// Platforms can override joinability using the "Joinable", which overrides the default check
		// Remote friend is responsible for updating their presence to have the joinable status
		if (!JoinablePresenceString.IsEmpty())
		{
			bInASession = (JoinablePresenceString == TEXT("true"));
		}
		else
		{
			bInASession = bIsPlayingThisGame && bHasConnectInformation;
		}

		// Skip invalid entries and ones that do not fit our current filters.
		if (NickName.Len() > 0 && CanAddUserToList(bIsOnline, bIsPlayingThisGame, bInASession))
		{
			// Add to list
			TSharedRef<FOnlineFriendSteam> Friend(new FOnlineFriendSteam(SteamPlayerId));
			FriendsList.Friends.Add(Friend);

			// Now fill in the friend info
			Friend->AccountData.Add(TEXT("nickname"), NickName);

			// Initially update/cache the user's presence
			Friend->GetPresence();
		}
	}
}

void FOnlineAsyncTaskSteamReadFriendsList::TriggerDelegates(void)
{
	FOnlineAsyncTask::TriggerDelegates();

	Delegate.ExecuteIfBound(LocalUserNum, true, EFriendsLists::ToString(FriendsListFilter), FString());
}
