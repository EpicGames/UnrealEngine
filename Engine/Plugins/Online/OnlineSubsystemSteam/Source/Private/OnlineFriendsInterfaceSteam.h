// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManager.h"
#include "Interfaces/OnlineFriendsInterface.h"
#include "Interfaces/OnlinePresenceInterface.h"
#include "OnlineSubsystemSteamPackage.h"
#include "OnlineSubsystemSteamPrivate.h" // IWYU pragma: keep
class FOnlineSubsystemSteam;

/**
 * Info associated with an online friend on the Steam service
 */
class FOnlineFriendSteam : 
	public FOnlineFriend
{
public:

	// FOnlineUser

	virtual FUniqueNetIdRef GetUserId() const override;
	virtual FString GetRealName() const override;
	virtual FString GetDisplayName(const FString& Platform = FString()) const override;
	virtual bool GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const override;

	// FOnlineFriend
	
	virtual EInviteStatus::Type GetInviteStatus() const override;
	virtual const FOnlineUserPresence& GetPresence() const override;

	// FOnlineFriendMcp

	/**
	 * Init/default constructor
	 */
	FOnlineFriendSteam(const CSteamID& InUserId=CSteamID());

	/** Virtual destructor to keep clang happy */
	virtual ~FOnlineFriendSteam() {};

	/**
	 * Get account data attribute
	 *
	 * @param Key account data entry key
	 * @param OutVal [out] value that was found
	 *
	 * @return true if entry was found
	 */
	inline bool GetAccountData(const FString& Key, FString& OutVal) const
	{
		const FString* FoundVal = AccountData.Find(Key);
		if (FoundVal != NULL)
		{
			OutVal = *FoundVal;
			return true;
		}
		return false;
	}

	/** User Id represented as a FUniqueNetId */
	FUniqueNetIdRef UserId;
	/** Any addition account data associated with the friend */
	TMap<FString, FString> AccountData;

	/** A backup presence we can use if we are unable to get a friend's presence for any reason. */
	static const FOnlineUserPresence EmptyPresence;
};

/**
 * Implements the Steam specific interface for friends
 */
class FOnlineFriendsSteam :
	public IOnlineFriends
{
	/** Reference to the main Steam subsystem */
	class FOnlineSubsystemSteam* SteamSubsystem;
	/** The steam user interface to use when interacting with steam */
	class ISteamUser* SteamUserPtr;
	/** The steam friends interface to use when interacting with steam */
	class ISteamFriends* SteamFriendsPtr;
	/** list of friends */
	struct FSteamFriendsList
	{
		TArray< TSharedRef<FOnlineFriendSteam> > Friends;
	};
	/** map of local user idx to friends */
	TMap<int32, FSteamFriendsList> FriendsLists;

	friend class FOnlineAsyncTaskSteamReadFriendsList;

PACKAGE_SCOPE:

	FOnlineFriendsSteam();

public:

	/**
	 * Initializes the various interfaces
	 *
	 * @param InSteamSubsystem the subsystem that owns this object
	 */
	FOnlineFriendsSteam(FOnlineSubsystemSteam* InSteamSubsystem);

	virtual ~FOnlineFriendsSteam() {};

	// IOnlineFriends

	virtual bool ReadFriendsList(int32 LocalUserNum, const FString& ListName, const FOnReadFriendsListComplete& Delegate = FOnReadFriendsListComplete()) override;
	virtual bool DeleteFriendsList(int32 LocalUserNum, const FString& ListName, const FOnDeleteFriendsListComplete& Delegate = FOnDeleteFriendsListComplete()) override;
	virtual bool SendInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName,  const FOnSendInviteComplete& Delegate = FOnSendInviteComplete()) override;
	virtual bool AcceptInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnAcceptInviteComplete& Delegate = FOnAcceptInviteComplete()) override;
 	virtual bool RejectInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual void SetFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FString& Alias, const FOnSetFriendAliasComplete& Delegate = FOnSetFriendAliasComplete()) override;
	virtual void DeleteFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnDeleteFriendAliasComplete& Delegate = FOnDeleteFriendAliasComplete()) override;
	virtual bool DeleteFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual bool GetFriendsList(int32 LocalUserNum, const FString& ListName, TArray< TSharedRef<FOnlineFriend> >& OutFriends) override;
	virtual TSharedPtr<FOnlineFriend> GetFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual bool IsFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual bool QueryRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace) override;
	virtual bool GetRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace, TArray< TSharedRef<FOnlineRecentPlayer> >& OutRecentPlayers) override;
	virtual void DumpRecentPlayers() const override;
	virtual bool BlockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId) override;
	virtual bool UnblockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId) override;
	virtual bool QueryBlockedPlayers(const FUniqueNetId& UserId) override;
	virtual bool GetBlockedPlayers(const FUniqueNetId& UserId, TArray< TSharedRef<FOnlineBlockedPlayer> >& OutBlockedPlayers) override;
	virtual void DumpBlockedPlayers() const override;
};

typedef TSharedPtr<FOnlineFriendsSteam, ESPMode::ThreadSafe> FOnlineFriendsSteamPtr;

/**
 * Fires the delegates for the task on the game thread
 */
class FOnlineAsyncTaskSteamReadFriendsList :
	public FOnlineAsyncTask
{
	/** Interface pointer to trigger the delegates on */
	FOnlineFriendsSteam* FriendsPtr;
	/** The user that is triggering the event */
	int32 LocalUserNum;
	/** The type of list we are trying to fetch */
	EFriendsLists::Type FriendsListFilter;

	FOnReadFriendsListComplete Delegate;

	bool CanAddUserToList(bool bIsOnline, bool bIsPlayingThisGame, bool bIsPlayingGameInSession);

public:
	/**
	 * Inits the pointer used to trigger the delegates on
	 *
	 * @param InFriendsPtr the interface to call the delegates on
	 * @param InLocalUserNum the local user that requested the read
	 * @param InDelegate the delegate that will be called when reading the friends list is complete
	 */
	FOnlineAsyncTaskSteamReadFriendsList(FOnlineFriendsSteam* InFriendsPtr, int32 InLocalUserNum, const FString& InFriendsListFilter, const FOnReadFriendsListComplete& InDelegate) :
		FriendsPtr(InFriendsPtr),
		LocalUserNum(InLocalUserNum),
		Delegate(InDelegate)
	{
		check(FriendsPtr);

		if (InFriendsListFilter.Equals(EFriendsLists::ToString(EFriendsLists::OnlinePlayers), ESearchCase::IgnoreCase))
		{
			FriendsListFilter = EFriendsLists::OnlinePlayers;
		}
		else if (InFriendsListFilter.Equals(EFriendsLists::ToString(EFriendsLists::InGamePlayers), ESearchCase::IgnoreCase))
		{
			FriendsListFilter = EFriendsLists::InGamePlayers;
		}
		else if (InFriendsListFilter.Equals(EFriendsLists::ToString(EFriendsLists::InGameAndSessionPlayers), ESearchCase::IgnoreCase))
		{
			FriendsListFilter = EFriendsLists::InGameAndSessionPlayers;
		}
		else
		{
			FriendsListFilter = EFriendsLists::Default;
		}
	}

	// FOnlineAsyncTask

	virtual FString ToString(void) const override
	{
		return TEXT("FOnlineFriendsSteam::ReadFriendsList() async task completed successfully");
	}

	virtual bool IsDone(void) const override
	{
		return true;
	}

	virtual bool WasSuccessful(void) const override
	{
		return true;
	}

	virtual void Finalize() override;
	virtual void TriggerDelegates(void) override;
};
