// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineIdentityInterfaceSteam.h"

#include "Misc/ConfigCacheIni.h"
#include "OnlineAuthInterfaceSteam.h"
#include "OnlineEncryptedAppTicketInterfaceSteam.h"
#include "OnlineSubsystemSteam.h"
#include "OnlineSubsystemSteamTypes.h"
#include "OnlineError.h"
#include <steam/isteamuser.h>

FOnlineIdentitySteam::FOnlineIdentitySteam(FOnlineSubsystemSteam* InSubsystem) :
	SteamUserPtr(NULL),
	SteamFriendsPtr(NULL),
	SteamSubsystem(InSubsystem)
{
	SteamUserPtr = SteamUser();
	SteamFriendsPtr = SteamFriends();
}

TSharedPtr<FUserOnlineAccount> FOnlineIdentitySteam::GetUserAccount(const FUniqueNetId& UserId) const
{
	//@todo - not implemented
	return NULL;
}
TArray<TSharedPtr<FUserOnlineAccount> > FOnlineIdentitySteam::GetAllUserAccounts() const 
{
	//@todo - not implemented
	return TArray<TSharedPtr<FUserOnlineAccount> >();
}

bool FOnlineIdentitySteam::Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials)
{
	FString ErrorStr;
	if (LocalUserNum < MAX_LOCAL_PLAYERS)
	{
		// Double check they are properly logged in
		if (SteamUserPtr != NULL &&
			// Login is handled by steam
			SteamUserPtr->BLoggedOn())
		{
			// Login changed delegate
			TriggerOnLoginChangedDelegates(LocalUserNum);
			// Login completion delegate
			TriggerOnLoginCompleteDelegates(LocalUserNum, true, *FUniqueNetIdSteam::Create(SteamUserPtr->GetSteamID()), TEXT(""));
			return true;
		}
		else
		{
			// User is not currently logged into Steam
			ErrorStr = TEXT("Not logged in or no connection.");
		}
	}
	else
	{
		// Requesting a local user is always invalid
		ErrorStr = FString::Printf(TEXT("Invalid user %d"),LocalUserNum);
	}

	if (!ErrorStr.IsEmpty())
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Failed Steam login. %s"), *ErrorStr);
		TriggerOnLoginCompleteDelegates(LocalUserNum, false, *FUniqueNetIdSteam::EmptyId(), ErrorStr);
	}

	return false;
}

bool FOnlineIdentitySteam::Logout(int32 LocalUserNum)
{
	TriggerOnLogoutCompleteDelegates(LocalUserNum,false);
	return false;
}

bool FOnlineIdentitySteam::AutoLogin(int32 LocalUserNum)
{
	if (!IsRunningDedicatedServer())
	{
		// Double check they are properly logged in
		if (SteamUserPtr != NULL &&
			// Login is handled by steam
			SteamUserPtr->BLoggedOn())
		{
			// Login changed delegate
			TriggerOnLoginChangedDelegates(LocalUserNum);
			// Login completion delegate
			FString AuthToken = GetAuthToken(LocalUserNum);
			TriggerOnLoginCompleteDelegates(LocalUserNum, true, *FUniqueNetIdSteam::Create(SteamUserPtr->GetSteamID()), TEXT(""));
			return true;
		}
		TriggerOnLoginCompleteDelegates(0, false, *FUniqueNetIdSteam::EmptyId(), TEXT("AutoLogin failed. Not logged in or no connection."));
		return false;
	}	
	else
	{
		// Autologin for dedicated servers happens via session creation in the GameServerAPI LogOnAnonymous()
		return false;
	}
}

ELoginStatus::Type FOnlineIdentitySteam::GetLoginStatus(int32 LocalUserNum) const
{
	if (LocalUserNum < MAX_LOCAL_PLAYERS &&
		SteamUserPtr != NULL)
	{
		return SteamUserPtr->BLoggedOn() ? ELoginStatus::LoggedIn : ELoginStatus::NotLoggedIn;
	}
	return ELoginStatus::NotLoggedIn; 
}

ELoginStatus::Type FOnlineIdentitySteam::GetLoginStatus(const FUniqueNetId& UserId) const 
{
	return GetLoginStatus(0);
}

FUniqueNetIdPtr FOnlineIdentitySteam::GetUniquePlayerId(int32 LocalUserNum) const
{
	if (LocalUserNum < MAX_LOCAL_PLAYERS &&
		SteamUserPtr != NULL)
	{
		return FUniqueNetIdSteam::Create(SteamUserPtr->GetSteamID());
	}
	return NULL;
}

FUniqueNetIdPtr FOnlineIdentitySteam::CreateUniquePlayerId(uint8* Bytes, int32 Size)
{
	if (Bytes && Size == sizeof(uint64))
	{
		uint64* RawUniqueId = (uint64*)Bytes;
		CSteamID SteamId(*RawUniqueId);
		if (SteamId.IsValid())
		{
			return FUniqueNetIdSteam::Create(SteamId);
		}
	}

	return NULL;
}

FUniqueNetIdPtr FOnlineIdentitySteam::CreateUniquePlayerId(const FString& Str)
{
	return FUniqueNetIdSteam::Create(Str);
}

/**
 * Reads the player's nick name from the online service
 *
 * @param LocalUserNum the controller number of the associated user
 *
 * @return a string containing the players nick name
 */
FString FOnlineIdentitySteam::GetPlayerNickname(int32 LocalUserNum) const
{
	if (LocalUserNum < MAX_LOCAL_PLAYERS &&
		SteamFriendsPtr != NULL)
	{
		const char* PersonaName = SteamFriendsPtr->GetPersonaName();
		return FString(UTF8_TO_TCHAR(PersonaName));
	}
	return FString(TEXT(""));
}

FString FOnlineIdentitySteam::GetPlayerNickname(const FUniqueNetId& UserId) const
{
	const FUniqueNetIdSteam& SteamId = (const FUniqueNetIdSteam&)UserId;
	if (SteamFriendsPtr != NULL)
	{
		FString PersonaName = UTF8_TO_TCHAR(SteamFriendsPtr->GetFriendPersonaName(SteamId));
		return PersonaName;
	}
	return FString(TEXT(""));
}

/**
 * Gets a user's platform specific authentication token to verify their identity
 *
 * @param LocalUserNum the controller number of the associated user
 *
 * @return String representing the authentication token
 */
FString FOnlineIdentitySteam::GetAuthToken(int32 LocalUserNum) const
{
	FString ResultToken;
	if (LocalUserNum < MAX_LOCAL_PLAYERS)
	{
		// Double check they are properly logged in
		if (SteamUserPtr != NULL &&
			SteamUserPtr->BLoggedOn())
		{
			uint8 AuthToken[1024];
			uint32 AuthTokenSize = 0;
			if (SteamUserPtr->GetAuthSessionTicket(AuthToken, UE_ARRAY_COUNT(AuthToken), &AuthTokenSize, nullptr) != k_HAuthTicketInvalid &&
				AuthTokenSize > 0)
			{
				ResultToken = BytesToHex(AuthToken, AuthTokenSize);
				UE_LOG_ONLINE_IDENTITY(Log, TEXT("Obtained steam authticket"));
				// In release builds our code checks the authTicket faster than Steam's login server can save it
				// Added a small amount of sleep here so the ResultToken is valid by the time this call returns
				FPlatformProcess::Sleep(0.1f);
			}
			else
			{
				UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Failed to acquire Steam auth session ticket for %d"), 
					LocalUserNum);
			}
		}
	}	
	return ResultToken;
}

void FOnlineIdentitySteam::RevokeAuthToken(const FUniqueNetId& UserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate)
{
	UE_LOG_ONLINE_IDENTITY(Display, TEXT("FOnlineIdentitySteam::RevokeAuthToken not implemented"));
	FUniqueNetIdRef UserIdRef(UserId.AsShared());
	SteamSubsystem->ExecuteNextTick([UserIdRef, Delegate]()
	{
		Delegate.ExecuteIfBound(*UserIdRef, FOnlineError(FString(TEXT("RevokeAuthToken not implemented"))));
	});
}

void FOnlineIdentitySteam::GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI)
{
	Delegate.ExecuteIfBound(UserId, Privilege, (uint32)EPrivilegeResults::NoFailures);
}

FPlatformUserId FOnlineIdentitySteam::GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& UniqueNetId) const
{
	for (int i = 0; i < MAX_LOCAL_PLAYERS; ++i)
	{
		auto CurrentUniqueId = GetUniquePlayerId(i);
		if (CurrentUniqueId.IsValid() && (*CurrentUniqueId == UniqueNetId))
		{
			return GetPlatformUserIdFromLocalUserNum(i);
		}
	}

	return PLATFORMUSERID_NONE;
}

FString FOnlineIdentitySteam::GetAuthType() const
{
	return TEXT("");
}

void FOnlineIdentitySteam::GetLinkedAccountAuthToken(int32 LocalUserNum, const FString& InTokenType, const FOnGetLinkedAccountAuthTokenCompleteDelegate& Delegate) const
{
	const TCHAR* AppTokenType = TEXT("App");
	const TCHAR* SessionTokenType = TEXT("Session");
	const TCHAR* WebAPITokenType = TEXT("WebAPI");

	FString TokenType = InTokenType;
	if (TokenType.IsEmpty())
	{
		// Default to APP tokens
		TokenType = AppTokenType;

		// But allow the user to configure something else
		GConfig->GetString(TEXT("OnlineSubsystemSteam"), TEXT("DefaultLinkedAccountAuthTokenType"), TokenType, GEngineIni);
	}

	if (TokenType == AppTokenType)
	{
		TSharedRef<FDelegateHandle> DelegateHandle = MakeShared<FDelegateHandle>();
		*DelegateHandle = SteamSubsystem->GetEncryptedAppTicketInterface()->OnEncryptedAppTicketResultDelegate.AddLambda([this, LocalUserNum, Delegate, DelegateHandle](bool bEncryptedDataAvailable, int32 ResultCode)
		{
			FExternalAuthToken ExternalToken;
			if (bEncryptedDataAvailable)
			{
				SteamSubsystem->GetEncryptedAppTicketInterface()->GetEncryptedAppTicket(ExternalToken.TokenData);
			}
			// Pass the info back to the original caller
			Delegate.ExecuteIfBound(LocalUserNum, ExternalToken.HasTokenData(), ExternalToken);
			SteamSubsystem->GetEncryptedAppTicketInterface()->OnEncryptedAppTicketResultDelegate.Remove(*DelegateHandle);
		});
		SteamSubsystem->GetEncryptedAppTicketInterface()->RequestEncryptedAppTicket(nullptr, 0);
	}
	else if (TokenType == SessionTokenType)
	{
		FExternalAuthToken AuthToken;
		AuthToken.TokenString = GetAuthToken(LocalUserNum);
		Delegate.ExecuteIfBound(LocalUserNum, AuthToken.HasTokenString(), AuthToken);
	}
	else if (TokenType.StartsWith(WebAPITokenType))
	{
		if (SteamUserPtr != NULL && SteamUserPtr->BLoggedOn())
		{
			// Split on ':' to allow client to specify web service; if not set use default specified in config
			FString RemoteServiceIdentity;
			if (!TokenType.Split(TEXT(":"), nullptr, &RemoteServiceIdentity))
			{
				GConfig->GetString(TEXT("OnlineSubsystemSteam"), TEXT("DefaultRemoteServiceIdentity"), RemoteServiceIdentity, GEngineIni);
			}

			if (RemoteServiceIdentity.IsEmpty())
			{
				UE_LOG_ONLINE_IDENTITY(Warning, TEXT("FOnlineIdentitySteam::GetLinkedAccountAuthToken DefaultRemoteServiceIdentity not set"));
				Delegate.ExecuteIfBound(LocalUserNum, false, FExternalAuthToken());
				return;
			}
			
			SteamSubsystem->GetAuthInterface()->GetAuthTicketForWebApi(RemoteServiceIdentity, FOnGetAuthTicketForWebApiCompleteDelegate::CreateLambda(
				[this, LocalUserNum, OnComplete = Delegate](uint32 AuthTicketHandle, const FString& TicketString)
				{
					FExternalAuthToken AuthToken;
					AuthToken.TokenString = TicketString;
					OnComplete.ExecuteIfBound(LocalUserNum, AuthToken.HasTokenString(), AuthToken);
				}));
		}
		else
		{
			if (SteamUserPtr == NULL)
			{
				UE_LOG_ONLINE_IDENTITY(Warning, TEXT("FOnlineIdentitySteam::GetLinkedAccountAuthToken failed because SteamUserPtr is NULL"));
			}
			else
			{
				UE_LOG_ONLINE_IDENTITY(Warning, TEXT("FOnlineIdentitySteam::GetLinkedAccountAuthToken failed because User is not logged into Steam"));
			}

			// fire the delegate with failure for cases like UE-330775
			Delegate.ExecuteIfBound(LocalUserNum, false, FExternalAuthToken());
		}
	}
	else
	{	
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("FOnlineIdentitySteam::GetLinkedAccountAuthToken TokenType=[%s] unknown"), *TokenType);
		Delegate.ExecuteIfBound(LocalUserNum, false, FExternalAuthToken());
	}
}

