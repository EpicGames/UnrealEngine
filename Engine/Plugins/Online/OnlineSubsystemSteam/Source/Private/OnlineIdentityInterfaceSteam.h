// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystemSteamPackage.h"

struct FPlatformUserId;

class FOnlineSubsystemSteam;

class FOnlineIdentitySteam :
	public IOnlineIdentity
{
	/** The steam user interface to use when interacting with steam */
	class ISteamUser* SteamUserPtr;
	/** The steam friends interface to use when interacting with steam */
	class ISteamFriends* SteamFriendsPtr;

	/** Cached pointer to owning subsystem */
	FOnlineSubsystemSteam* SteamSubsystem;

PACKAGE_SCOPE:

	FOnlineIdentitySteam(FOnlineSubsystemSteam* InSubsystem);
	
public:
	virtual ~FOnlineIdentitySteam() {};

	// IOnlineIdentity

	virtual bool Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials) override;
	virtual bool Logout(int32 LocalUserNum) override;
	virtual bool AutoLogin(int32 LocalUserNum) override;
	virtual TSharedPtr<FUserOnlineAccount> GetUserAccount(const FUniqueNetId& UserId) const override;
	virtual TArray<TSharedPtr<FUserOnlineAccount> > GetAllUserAccounts() const override;
	virtual FUniqueNetIdPtr GetUniquePlayerId(int32 LocalUserNum) const override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(uint8* Bytes, int32 Size) override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(const FString& Str) override;
	virtual ELoginStatus::Type GetLoginStatus(int32 LocalUserNum) const override;
	virtual ELoginStatus::Type GetLoginStatus(const FUniqueNetId& UserId) const override;
	virtual FString GetPlayerNickname(int32 LocalUserNum) const override;
	virtual FString GetPlayerNickname(const FUniqueNetId& UserId) const override;
	virtual FString GetAuthToken(int32 LocalUserNum) const override;
	virtual void RevokeAuthToken(const FUniqueNetId& UserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate) override;
	virtual void GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI=EShowPrivilegeResolveUI::Default) override;
	virtual FPlatformUserId GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& UniqueNetId) const override;
	virtual FString GetAuthType() const override;
	virtual void GetLinkedAccountAuthToken(int32 LocalUserNum, const FString& TokenType, const FOnGetLinkedAccountAuthTokenCompleteDelegate& Delegate) const override;
};

typedef TSharedPtr<FOnlineIdentitySteam, ESPMode::ThreadSafe> FOnlineIdentitySteamPtr;
