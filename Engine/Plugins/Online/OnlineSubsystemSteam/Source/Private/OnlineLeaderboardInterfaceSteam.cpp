// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineLeaderboardInterfaceSteam.h"
#include "Misc/ScopeLock.h"
#include "OnlineSubsystemSteam.h"
#include "OnlineAsyncTaskManagerSteam.h"
#include "SteamUtilities.h"
#include "OnlineAchievementsInterfaceSteam.h"

/**
 *	Create the proper stat name for a given leaderboard/stat combination
 *
 * @param LeaderboardName name of leaderboard 
 * @param StatName name of stat
 */
inline FString GetLeaderboardStatName(const FString& LeaderboardName, const FString& StatName)
{
	return FString::Printf(TEXT("%s_%s"), *LeaderboardName, *StatName);
}

/** Helper function to convert enums */
inline ELeaderboardSortMethod ToSteamLeaderboardSortMethod(ELeaderboardSort::Type InSortMethod)
{			
	switch(InSortMethod)
	{
	case ELeaderboardSort::Ascending:
		return k_ELeaderboardSortMethodAscending;
	case ELeaderboardSort::Descending:
		return k_ELeaderboardSortMethodDescending;
	case ELeaderboardSort::None:
	default:
		return k_ELeaderboardSortMethodNone;
	}
}

/** Helper function to convert enums */
inline ELeaderboardSort::Type FromSteamLeaderboardSortMethod(ELeaderboardSortMethod InSortMethod)
{
	switch(InSortMethod)
	{
	case k_ELeaderboardSortMethodAscending:
		return ELeaderboardSort::Ascending;
	case  k_ELeaderboardSortMethodDescending:
		return ELeaderboardSort::Descending;
	case k_ELeaderboardSortMethodNone:
	default:
		return ELeaderboardSort::None;
	}
}

/** Helper function to convert enums */
inline ELeaderboardDisplayType ToSteamLeaderboardDisplayType(ELeaderboardFormat::Type InDisplayFormat)
{
	switch(InDisplayFormat)
	{
	case ELeaderboardFormat::Seconds:
		return k_ELeaderboardDisplayTypeTimeSeconds;
	case ELeaderboardFormat::Milliseconds:
		return k_ELeaderboardDisplayTypeTimeMilliSeconds;
	case ELeaderboardFormat::Number:
	default:
		return k_ELeaderboardDisplayTypeNumeric;
	}
}

/** Helper function to convert enums */
inline ELeaderboardFormat::Type FromSteamLeaderboardDisplayType(ELeaderboardDisplayType InDisplayFormat)
{
	switch(InDisplayFormat)
	{
	case k_ELeaderboardDisplayTypeTimeSeconds:
		return ELeaderboardFormat::Seconds;
	case k_ELeaderboardDisplayTypeTimeMilliSeconds:
		return ELeaderboardFormat::Milliseconds;
	case k_ELeaderboardDisplayTypeNumeric:
	default:
		return ELeaderboardFormat::Number;
	}
}

/**
 *	Async task to retrieve all stats for a single user from the Steam backend
 */
class FOnlineAsyncTaskSteamRequestUserStats : public FOnlineAsyncTaskSteam
{
private:

	/** Has this task been initialized yet */
	bool bInit;
	/** User Id we are requesting stats for */
	FUniqueNetIdSteamRef UserId;
	/** Returned results from Steam */
	UserStatsReceived_t CallbackResults;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamRequestUserStats() = delete;

public:

	FOnlineAsyncTaskSteamRequestUserStats(FOnlineSubsystemSteam* InSteamSubsystem, const FUniqueNetIdSteam& InUserId) :
		FOnlineAsyncTaskSteam(InSteamSubsystem, k_uAPICallInvalid),
		bInit(false),
		UserId(InUserId.AsShared())
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncTaskSteamRequestUserStats bWasSuccessful: %d UserId: %s"), WasSuccessful(), *UserId->ToDebugString());
	}

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override
	{
		if (!bInit)
		{
			ISteamUserStats* SteamUserStatsPtr = SteamUserStats();
			check(SteamUserStatsPtr);
			CallbackHandle = SteamUserStatsPtr->RequestUserStats(*(uint64*)UserId->GetBytes());
			bInit = true;
		}

		if (CallbackHandle != k_uAPICallInvalid)
		{
			ISteamUtils* SteamUtilsPtr = SteamUtils();
			check(SteamUtilsPtr);
			bool bFailedCall = false; 

			// Poll for completion status
			bIsComplete = SteamUtilsPtr->IsAPICallCompleted(CallbackHandle, &bFailedCall) ? true : false; 
			if (bIsComplete) 
			{ 
				bool bFailedResult;
				// Retrieve the callback data from the request
				bool bSuccessCallResult = SteamUtilsPtr->GetAPICallResult(CallbackHandle, &CallbackResults, sizeof(CallbackResults), CallbackResults.k_iCallback, &bFailedResult); 
				bWasSuccessful = (bSuccessCallResult ? true : false) &&
					(!bFailedCall ? true : false) &&
					(!bFailedResult ? true : false);
			} 
		}
		else
		{
			bWasSuccessful = false;
			bIsComplete = false;
		}
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override
	{
		FOnlineAsyncTaskSteam::Finalize();

		if (bWasSuccessful)
		{
			const CGameID GameID(Subsystem->GetSteamAppId());
			if (GameID.ToUint64() == CallbackResults.m_nGameID)
			{
				check(CallbackResults.m_steamIDUser == *UserId);
				if (CallbackResults.m_eResult != k_EResultOK)
				{
					if (CallbackResults.m_eResult == k_EResultFail)
					{
						UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Failed to obtain steam user stats, user: %s has no stats entries"), *UserId->ToDebugString());
					}
					else
					{
						UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Failed to obtain steam user stats, user: %s error: %s"), *UserId->ToDebugString(), 
							*SteamResultString(CallbackResults.m_eResult));
					}
				}

				bWasSuccessful = (CallbackResults.m_eResult == k_EResultOK) ? true : false;
			}
			else
			{
				UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Obtained steam user stats, but for wrong game! Ignoring."));
			}
		}
		else
		{
			UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Failed to obtain steam user stats, user: %s error: unknown"), *UserId->ToDebugString());
		}
	}
};

/**
 *	Async task to update a single user's stats on the Steam backend
 */
class FOnlineAsyncTaskSteamUpdateStats : public FOnlineAsyncTaskSteam
{
private:
	/** Has this task been initialized yet */
	bool bInit;
	/** Player whose stats are updating */
	FUniqueNetIdSteamRef UserId;
	/** Array of stats to update for the given user */
	const FStatPropertyArray Stats;
	/** Returned results from Steam */
	UserStatsReceived_t CallbackResults;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamUpdateStats() = delete;

public:
	FOnlineAsyncTaskSteamUpdateStats(FOnlineSubsystemSteam* InSteamSubsystem, const FUniqueNetIdSteam& InUserId, const FStatPropertyArray& InStats) :
		FOnlineAsyncTaskSteam(InSteamSubsystem, k_uAPICallInvalid),
		bInit(false),
		UserId(InUserId.AsShared()),
		Stats(InStats)
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncTaskSteamUpdateStats bWasSuccessful: %d User: %s"), WasSuccessful(), *UserId->ToDebugString());
	}

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override
	{
		if (!bInit)
		{
			// Triggers a Steam event async to let us know when the stats are available
			CallbackHandle = SteamUserStats()->RequestUserStats(*UserId);
			bInit = true;
		}

		if (CallbackHandle != k_uAPICallInvalid)
		{
			ISteamUtils* SteamUtilsPtr = SteamUtils();
			bool bFailedCall = false; 

			// Poll for completion status
			bIsComplete = SteamUtilsPtr->IsAPICallCompleted(CallbackHandle, &bFailedCall) ? true : false; 
			if (bIsComplete) 
			{ 
				bool bFailedResult;
				// Retrieve the callback data from the request
				bool bSuccessCallResult = SteamUtilsPtr->GetAPICallResult(CallbackHandle, &CallbackResults, sizeof(CallbackResults), CallbackResults.k_iCallback, &bFailedResult); 
				bWasSuccessful = (bSuccessCallResult ? true : false) &&
					(!bFailedCall ? true : false) &&
					(!bFailedResult ? true : false) &&
					((CallbackResults.m_eResult == k_EResultOK) ? true : false);

				if (bWasSuccessful)
				{
					// Stats are set here to ensure that this happens before any possible call to StoreStats()
					ISteamUserStats* SteamUserStatsPtr = SteamUserStats();
					check(SteamUserStatsPtr);
					for (FStatPropertyArray::TConstIterator It(Stats); It; ++It)
					{
						bool bSuccess = false;
						const FString StatName = It.Key();
						const FVariantData& Stat = It.Value();

						switch (Stat.GetType())
						{
						case EOnlineKeyValuePairDataType::Int32:
							{
								int32 Value;
								Stat.GetValue(Value);
								bSuccess = SteamUserStatsPtr->SetStat(TCHAR_TO_UTF8(*StatName), Value) ? true : false;
								break;
							}

						case EOnlineKeyValuePairDataType::Float:
							{
								float Value;
								Stat.GetValue(Value);
								bSuccess = SteamUserStatsPtr->SetStat(TCHAR_TO_UTF8(*StatName), Value) ? true : false;
								break;
							}
						default:
							UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Skipping unsuppported key value pair uploading to Steam %s=%s"), *StatName, *Stat.ToString());
							break;
						}

						if (!bSuccess)
						{
							UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Failure to write key value pair when uploading to Steam %s=%s"), *StatName, *Stat.ToString());
							bWasSuccessful = false;
						}
					}
				}
			} 
		}
		else
		{
			bWasSuccessful = false;
			bIsComplete = true;
		}
	}
};

/**
 *	Async task to retrieve a single user's stats from Steam
 */
class FOnlineAsyncTaskSteamRetrieveStats : public FOnlineAsyncTaskSteam
{
private:

	/** Has this task been initialized yet */
	bool bInit;
	/** User to retrieve stats for */
	FUniqueNetIdSteamRef UserId;
	/** Handle to the read object where the data will be stored */
	FOnlineLeaderboardReadPtr ReadObject;
	/** Returned results from Steam */
	UserStatsReceived_t CallbackResults;
	/** Potentially multiple user requests involved in filling in the read object, should this one trigger the finished delegate */
	bool bShouldTriggerDelegates;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamRetrieveStats() = delete;

public:

	FOnlineAsyncTaskSteamRetrieveStats(FOnlineSubsystemSteam* InSteamSubsystem, const FUniqueNetIdSteam& InUserId, const FOnlineLeaderboardReadRef& InReadObject, bool bInShouldTriggerDelegates) :
		FOnlineAsyncTaskSteam(InSteamSubsystem, k_uAPICallInvalid),
		bInit(false),
		UserId(InUserId.AsShared()),
		ReadObject(InReadObject),
		bShouldTriggerDelegates(bInShouldTriggerDelegates)
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncTaskSteamRetrieveStats bWasSuccessful: %d UserId: %s"), WasSuccessful(), *UserId->ToDebugString());
	}

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override
	{
		if (!bInit)
		{
			// Triggers a Steam event async to let us know when the stats are available
			CallbackHandle = SteamUserStats()->RequestUserStats(*UserId);
			bInit = true;
		}

		if (CallbackHandle != k_uAPICallInvalid)
		{
			ISteamUtils* SteamUtilsPtr = SteamUtils();
			bool bFailedCall = false; 

			// Poll for completion status
			bIsComplete = SteamUtilsPtr->IsAPICallCompleted(CallbackHandle, &bFailedCall) ? true : false; 
			if (bIsComplete) 
			{ 
				bool bFailedResult;
				// Retrieve the callback data from the request
				bool bSuccessCallResult = SteamUtilsPtr->GetAPICallResult(CallbackHandle, &CallbackResults, sizeof(CallbackResults), CallbackResults.k_iCallback, &bFailedResult); 
				bWasSuccessful = (bSuccessCallResult ? true : false) &&
					(!bFailedCall ? true : false) &&
					(!bFailedResult ? true : false);
			} 
		}
		else
		{
			bWasSuccessful = false;
			bIsComplete = true;
		}
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override
	{
		FOnlineAsyncTaskSteam::Finalize();

		ISteamUserStats* SteamUserStatsPtr = SteamUserStats();
		check(SteamUserStatsPtr);

		FOnlineStatsRow* UserRow = ReadObject->FindPlayerRecord(*UserId);
		if (UserRow == NULL)
		{
			const FString NickName(UTF8_TO_TCHAR(SteamFriends()->GetFriendPersonaName(*UserId)));
			UserRow = new (ReadObject->Rows) FOnlineStatsRow(NickName, UserId);
		}

		if (bWasSuccessful)
		{
			if (CallbackResults.m_eResult != k_EResultOK)
			{
				// Append empty data for this user
				FVariantData EmptyData;
				for (int32 StatIdx = 0; StatIdx < ReadObject->ColumnMetadata.Num(); StatIdx++)
				{
					const FColumnMetaData& ColumnMeta = ReadObject->ColumnMetadata[StatIdx];
					UserRow->Columns.Add(ColumnMeta.ColumnName, EmptyData);
				}
			}
			else
			{
				for (int32 StatIdx = 0; StatIdx < ReadObject->ColumnMetadata.Num(); StatIdx++)
				{
					const FColumnMetaData& ColumnMeta = ReadObject->ColumnMetadata[StatIdx];
					const FString LeaderboardStat = GetLeaderboardStatName(ReadObject->LeaderboardName, ColumnMeta.ColumnName);
					const FString StatName = LeaderboardStat;

					bool bSuccess = false;
					FVariantData* LastColumn = NULL;
					switch (ColumnMeta.DataType)
					{
					case EOnlineKeyValuePairDataType::Int32:
						{
							int32 Value;
							bSuccess = SteamUserStatsPtr->GetUserStat(*UserId, TCHAR_TO_UTF8(*StatName), &Value) ? true : false;
							LastColumn = &(UserRow->Columns.Add(ColumnMeta.ColumnName, FVariantData(Value)));
							break;
						}

					case EOnlineKeyValuePairDataType::Float:
						{
							float Value;
							bSuccess = SteamUserStatsPtr->GetUserStat(*UserId, TCHAR_TO_UTF8(*StatName), &Value) ? true : false;
							LastColumn = &(UserRow->Columns.Add(ColumnMeta.ColumnName, FVariantData(Value)));
							break;
						}
					default:
						UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Unsupported key value pair during retrieval from Steam %s"), *StatName);
						LastColumn = &(UserRow->Columns.Add(ColumnMeta.ColumnName, FVariantData()));
						break;
					}

					if (!bSuccess)
					{
						UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Failure to read key value pair during retrieval from Steam %s"), *StatName);
						LastColumn->Empty();
						bWasSuccessful = false;
					}
				}
			}
		}

		// Update the read state of this object
		ReadObject->ReadState = (bWasSuccessful && ReadObject->ReadState != EOnlineAsyncTaskState::Failed) ? EOnlineAsyncTaskState::Done : EOnlineAsyncTaskState::Failed;
	}

	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override 
	{
		FOnlineAsyncTaskSteam::TriggerDelegates();
		
		if (bShouldTriggerDelegates)
		{
			FOnlineLeaderboardsSteamPtr Leaderboards = StaticCastSharedPtr<FOnlineLeaderboardsSteam>(Subsystem->GetLeaderboardsInterface());
			Leaderboards->TriggerOnLeaderboardReadCompleteDelegates(ReadObject->ReadState == EOnlineAsyncTaskState::Done ? true : false);
		}
	}
};

/**
 *	Async task to retrieve a single user's stats from Steam
 *  Services both achievements themselves as well as achievement descriptions.
 */
class FOnlineAsyncTaskSteamGetAchievements : public FOnlineAsyncTaskSteam
{
public:

private:

	/** Has this task been initialized yet */
	bool bInit;
	/** User to retrieve stats for */
	FUniqueNetIdSteamRef UserId;
	/** Returned results from Steam */
	UserStatsReceived_t CallbackResults;
	/** Delegate for achievements */
	FOnQueryAchievementsCompleteDelegate AchievementDelegate;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamGetAchievements() = delete;

public:

	FOnlineAsyncTaskSteamGetAchievements(
		FOnlineSubsystemSteam* InSteamSubsystem, 
		const FUniqueNetIdSteam& InUserId, 
		const FOnQueryAchievementsCompleteDelegate& InAchievementDelegate ) :
		FOnlineAsyncTaskSteam(InSteamSubsystem, k_uAPICallInvalid),
		bInit(false),
		UserId(InUserId.AsShared()),
		AchievementDelegate(InAchievementDelegate)
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncTaskSteamGetAchievements bWasSuccessful: %d UserId: %s"), WasSuccessful(), *UserId->ToDebugString());
	}

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override
	{
		if (!bInit)
		{
			// Triggers a Steam event async to let us know when the stats are available
			CallbackHandle = SteamUserStats()->RequestUserStats(*UserId);
			bInit = true;
		}

		if (CallbackHandle != k_uAPICallInvalid)
		{
			ISteamUtils* SteamUtilsPtr = SteamUtils();
			bool bFailedCall = false; 

			// Poll for completion status
			bIsComplete = SteamUtilsPtr->IsAPICallCompleted(CallbackHandle, &bFailedCall) ? true : false; 
			if (bIsComplete) 
			{ 
				bool bFailedResult;
				// Retrieve the callback data from the request
				bool bSuccessCallResult = SteamUtilsPtr->GetAPICallResult(CallbackHandle, &CallbackResults, sizeof(CallbackResults), CallbackResults.k_iCallback, &bFailedResult); 
				bWasSuccessful = (bSuccessCallResult ? true : false) &&
					(!bFailedCall ? true : false) &&
					(!bFailedResult ? true : false) &&
					CallbackResults.m_eResult == k_EResultOK;
			} 
		}
		else
		{
			bWasSuccessful = false;
			bIsComplete = true;
		}
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override
	{
		FOnlineAsyncTaskSteam::Finalize();

		FOnlineAchievementsSteamPtr Achievements = StaticCastSharedPtr<FOnlineAchievementsSteam>(Subsystem->GetAchievementsInterface());
		Achievements->UpdateAchievementsForUser(*UserId, bWasSuccessful);
	}

	/**
	 *	Async task is given a chance to trigger its delegates
	 */
	virtual void TriggerDelegates() override 
	{
		FOnlineAsyncTaskSteam::TriggerDelegates();
		
		FOnlineAchievementsSteamPtr Achievements = StaticCastSharedPtr<FOnlineAchievementsSteam>(Subsystem->GetAchievementsInterface());
		AchievementDelegate.ExecuteIfBound(*UserId, bWasSuccessful);
	}
};

/**
 *	Async task to retrieve a Steam leaderboard, possibly creating it in the process
 *  The game must first retrieve the Leaderboard handle from the backend before reading/writing
 */
class FOnlineAsyncTaskSteamRetrieveLeaderboard : public FOnlineAsyncTaskSteam
{
private:

	/** Has this request been started */
	bool bInit;
	/** Name of requested leaderboard */
	FString LeaderboardName;
	/** Method of sorting the scores on the leaderboard */
	ELeaderboardSort::Type SortMethod;
	/** Method of displaying the data on the leaderboard */
	ELeaderboardFormat::Type DisplayFormat;
	/** Results returned from Steam backend */
	LeaderboardFindResult_t CallbackResults;
	/** Should find only */
	bool bFindOnly;

	/**
	 *	Actually create and find a leaderboard with the Steam backend
	 * If the leaderboard already exists, the leaderboard data will still be retrieved
	 * @param LeaderboardName name of leaderboard to create
	 * @param SortMethod method the leaderboard scores will be sorted, ignored if leaderboard exists
	 * @param DisplayFormat type of data the leaderboard represents, ignored if leaderboard exists
	 */
	void CreateOrFindLeaderboard(const FString& InLeaderboardName, ELeaderboardSort::Type InSortMethod, ELeaderboardFormat::Type InDisplayFormat)
	{
		if (bFindOnly)
		{
			CallbackHandle = SteamUserStats()->FindLeaderboard(TCHAR_TO_UTF8(*InLeaderboardName));
		}
		else
		{
			ELeaderboardSortMethod SortMethodSteam = ToSteamLeaderboardSortMethod(InSortMethod);
			ELeaderboardDisplayType DisplayTypeSteam = ToSteamLeaderboardDisplayType(InDisplayFormat);
			CallbackHandle = SteamUserStats()->FindOrCreateLeaderboard(TCHAR_TO_UTF8(*InLeaderboardName), SortMethodSteam, DisplayTypeSteam);
		}
	}

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamRetrieveLeaderboard() :
		FOnlineAsyncTaskSteam(NULL, k_uAPICallInvalid),
		bInit(false),
		SortMethod(ELeaderboardSort::Ascending),
		DisplayFormat(ELeaderboardFormat::Number),
		bFindOnly(true)
	{
	}

public:
	/** Create a leaderboard implementation */
	FOnlineAsyncTaskSteamRetrieveLeaderboard(FOnlineSubsystemSteam* InSteamSubsystem, const FString& InLeaderboardName, ELeaderboardSort::Type InSortMethod, ELeaderboardFormat::Type InDisplayFormat) :
		FOnlineAsyncTaskSteam(InSteamSubsystem, k_uAPICallInvalid),
		bInit(false),
		LeaderboardName(InLeaderboardName),
		SortMethod(InSortMethod),
		DisplayFormat(InDisplayFormat),
		bFindOnly(false)
	{
	}

	/** Find a leaderboard implementation */
	FOnlineAsyncTaskSteamRetrieveLeaderboard(FOnlineSubsystemSteam* InSteamSubsystem, const FString& InLeaderboardName) :
		FOnlineAsyncTaskSteam(InSteamSubsystem, k_uAPICallInvalid),
		bInit(false),
		LeaderboardName(InLeaderboardName),
		SortMethod(ELeaderboardSort::Ascending),
		DisplayFormat(ELeaderboardFormat::Number),
		bFindOnly(true)
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncTaskSteamRetrieveLeaderboard bWasSuccessful: %d"), WasSuccessful());
	}

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override
	{
		ISteamUtils* SteamUtilsPtr = SteamUtils();
		check(SteamUtilsPtr);

		if (!bInit)
		{
			CreateOrFindLeaderboard(LeaderboardName, SortMethod, DisplayFormat);
			bInit = true;
		}

		if (CallbackHandle != k_uAPICallInvalid)
		{
			bool bFailedCall = false; 

			// Poll for completion status
			bIsComplete = SteamUtilsPtr->IsAPICallCompleted(CallbackHandle, &bFailedCall) ? true : false; 
			if (bIsComplete) 
			{ 
				bool bFailedResult;
				// Retrieve the callback data from the request
				bool bSuccessCallResult = SteamUtilsPtr->GetAPICallResult(CallbackHandle, &CallbackResults, sizeof(CallbackResults), CallbackResults.k_iCallback, &bFailedResult); 
				bWasSuccessful = (bSuccessCallResult ? true : false) &&
					(!bFailedCall ? true : false) &&
					(!bFailedResult ? true : false) &&
					((CallbackResults.m_bLeaderboardFound != 0) ? true : false);
			} 
		}
		else
		{
			// Invalid API call
			bIsComplete = true;
			bWasSuccessful = false;
		}
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override
	{
		FOnlineAsyncTaskSteam::Finalize();

		// Copy the leaderboard handle into the array of data
		FOnlineLeaderboardsSteamPtr Leaderboards = StaticCastSharedPtr<FOnlineLeaderboardsSteam>(Subsystem->GetLeaderboardsInterface());

		FScopeLock ScopeLock(&Leaderboards->LeaderboardMetadataLock);
		FLeaderboardMetadataSteam* Leaderboard = Leaderboards->GetLeaderboardMetadata(LeaderboardName);
		check(Leaderboard);

		if (bWasSuccessful)
		{
			ISteamUserStats* SteamUserPtr = SteamUserStats();
			check(LeaderboardName == FString(SteamUserPtr->GetLeaderboardName(CallbackResults.m_hSteamLeaderboard)));

			Leaderboard->LeaderboardHandle = CallbackResults.m_hSteamLeaderboard;
			Leaderboard->TotalLeaderboardRows = SteamUserPtr->GetLeaderboardEntryCount(CallbackResults.m_hSteamLeaderboard);
			Leaderboard->DisplayFormat = FromSteamLeaderboardDisplayType(SteamUserPtr->GetLeaderboardDisplayType(CallbackResults.m_hSteamLeaderboard));
			Leaderboard->SortMethod = FromSteamLeaderboardSortMethod(SteamUserPtr->GetLeaderboardSortMethod(CallbackResults.m_hSteamLeaderboard));
			Leaderboard->AsyncState = EOnlineAsyncTaskState::Done;
		}
		else
		{
			Leaderboard->LeaderboardHandle = -1;
			Leaderboard->TotalLeaderboardRows = 0;
			Leaderboard->AsyncState = EOnlineAsyncTaskState::Failed;
		}
	}
};

/**
 *	Async task to retrieve actual arbitrary leaderboard entries from Steam (not the supporting stats/columns)
 *  The game must first retrieve the Leaderboard handle from the backend before reading/writing
 */
class FOnlineAsyncTaskSteamRetrieveLeaderboardEntries : public FOnlineAsyncTaskSteam
{
private:
	enum RetrieveType
	{
		None,
		// Fetch data about a group of arbitrary users
		FetchUsers,
		// Fetch data about the user's friends
		FetchFriends,
		// Fetch data around an arbitrary rank
		FetchRank,
		// Fetch data around the current user
		FetchCurRankUser,
		// Fetch data around an arbitrary user
		FetchRankUser,
		Max
	};

	/** Has this request been started */
	bool bInit;
	/** Players to request leaderboard data for */
	TArray< FUniqueNetIdRef > Players;
	/** Handle to the read object where the data will be stored */
	FOnlineLeaderboardReadRef ReadObject;
	/** Results from callback */
	LeaderboardScoresDownloaded_t CallbackResults;
	/** Type */
	RetrieveType Type;
	/** Rank/Range Query data */
	int32 Rank;
	int32 Range;
	/** If delegates should be triggered */
	bool bShouldTriggerDelegates;


public:
	FOnlineAsyncTaskSteamRetrieveLeaderboardEntries(FOnlineSubsystemSteam* InSteamSubsystem, const TArray< FUniqueNetIdRef >& InPlayers, const FOnlineLeaderboardReadRef& InReadObject) :
		FOnlineAsyncTaskSteam(InSteamSubsystem, k_uAPICallInvalid),
		bInit(false),
		Players(InPlayers),
		ReadObject(InReadObject),
		Type(RetrieveType::FetchUsers),
		bShouldTriggerDelegates(false)
	{
	}

	FOnlineAsyncTaskSteamRetrieveLeaderboardEntries(FOnlineSubsystemSteam* InSteamSubsystem, int32 InRank, int32 InRange, const FOnlineLeaderboardReadRef& InReadObject) :
		FOnlineAsyncTaskSteam(InSteamSubsystem, k_uAPICallInvalid),
		bInit(false),
		ReadObject(InReadObject),
		Type(RetrieveType::FetchRank),
		Rank(InRank),
		Range(InRange),
		bShouldTriggerDelegates(false)
	{
	}

	FOnlineAsyncTaskSteamRetrieveLeaderboardEntries(FOnlineSubsystemSteam* InSteamSubsystem, const FOnlineLeaderboardReadRef& InReadObject) :
		FOnlineAsyncTaskSteam(InSteamSubsystem, k_uAPICallInvalid),
		bInit(false),
		ReadObject(InReadObject),
		Type(RetrieveType::FetchFriends),
		bShouldTriggerDelegates(false)
	{
	}

	FOnlineAsyncTaskSteamRetrieveLeaderboardEntries(FOnlineSubsystemSteam* InSteamSubsystem, FUniqueNetIdRef InUser, int32 InRange, const FOnlineLeaderboardReadRef& InReadObject) :
		FOnlineAsyncTaskSteam(InSteamSubsystem, k_uAPICallInvalid),
		bInit(false),
		ReadObject(InReadObject),
		Range(InRange),
		bShouldTriggerDelegates(false)
	{
		Players.Push(FUniqueNetIdSteam::Create(*InUser));
		Type = Subsystem->IsLocalPlayer(*Players[0]) ? RetrieveType::FetchCurRankUser : RetrieveType::FetchRankUser;
	}

	FString TaskTypeToString() const
	{
		switch (Type)
		{
			default:
				return TEXT("Invalid");
			case FetchUsers:
				return TEXT("Fetch Users");
			case FetchFriends:
				return TEXT("Fetch Friends");
			case FetchRank:
				return TEXT("Fetch Global Ranks");
			case FetchCurRankUser:
				return TEXT("Fetch Rank around current user");
			case FetchRankUser:
				return TEXT("Fetch Rank around the users");
		}
	}

	/**
	 *	Get a human readable description of task
	 */
	FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncTaskSteamRetrieveLeaderboardEntries Task Type %s bWasSuccessful: %d"), *TaskTypeToString(), WasSuccessful());
	}

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override
	{
		ISteamUtils* SteamUtilsPtr = SteamUtils();
		check(SteamUtilsPtr);

		if (!bInit)
		{
			// Poll for leaderboard handle
			SteamLeaderboard_t LeaderboardHandle = -1;
			{
				FOnlineLeaderboardsSteamPtr Leaderboards = StaticCastSharedPtr<FOnlineLeaderboardsSteam>(Subsystem->GetLeaderboardsInterface());
				FScopeLock ScopeLock(&Leaderboards->LeaderboardMetadataLock);
				FLeaderboardMetadataSteam* Leaderboard = Leaderboards->GetLeaderboardMetadata(ReadObject->LeaderboardName);
				if (Leaderboard)
				{
					LeaderboardHandle = Leaderboard->LeaderboardHandle;
				}
			}

			if (LeaderboardHandle != -1)
			{
				ISteamUserStats* SteamUserStatsPtr = SteamUserStats();
				switch (Type)
				{
					default:
					case RetrieveType::FetchUsers:
					case RetrieveType::FetchRankUser:
					{
						if (Players.Num() > 0)
						{
							// Max leaderboard entries is 100
							int32 NumUsers = FPlatformMath::Min(Players.Num(), 100);
							CSteamID* IdArray = new CSteamID[NumUsers];
							for (int32 UserIdx = 0; UserIdx<NumUsers; UserIdx++)
							{
								IdArray[UserIdx] = *(uint64*)Players[UserIdx]->GetBytes();
							}
							CallbackHandle = SteamUserStatsPtr->DownloadLeaderboardEntriesForUsers(LeaderboardHandle, IdArray, NumUsers);

							delete[] IdArray;
						}
					} break;
					case RetrieveType::FetchCurRankUser:
					{
						CallbackHandle = SteamUserStatsPtr->DownloadLeaderboardEntries(LeaderboardHandle, k_ELeaderboardDataRequestGlobalAroundUser, -Range, Range);
					} break;
					case RetrieveType::FetchRank:
					{
						// Because of how Steam works, Start point will be just the rank itself, and range will only give you values after that rank
						// To fit with the definition of the rank lookup function, we need to adjust the offsets so we get a nice even distribution.
						int32 AdjustedStart = ((Rank - Range) <= 0) ? 1 : (Rank - Range);
						CallbackHandle = SteamUserStatsPtr->DownloadLeaderboardEntries(LeaderboardHandle, k_ELeaderboardDataRequestGlobal, AdjustedStart, (Rank + Range));
					} break;
					case RetrieveType::FetchFriends:
					{
						CallbackHandle = SteamUserStatsPtr->DownloadLeaderboardEntries(LeaderboardHandle, k_ELeaderboardDataRequestFriends, 0, 0);
					} break;
				}

				bInit = true;
			}
		}

		if (CallbackHandle != k_uAPICallInvalid)
		{
			bool bFailedCall = false; 

			// Poll for completion status
			bIsComplete = SteamUtilsPtr->IsAPICallCompleted(CallbackHandle, &bFailedCall) ? true : false; 
			if (bIsComplete) 
			{ 
				bool bFailedResult;
				// Retrieve the callback data from the request
				bool bSuccessCallResult = SteamUtilsPtr->GetAPICallResult(CallbackHandle, &CallbackResults, sizeof(CallbackResults), CallbackResults.k_iCallback, &bFailedResult); 
				bWasSuccessful = (bSuccessCallResult ? true : false) &&
					(!bFailedCall ? true : false) &&
					(!bFailedResult ? true : false) &&
					((CallbackResults.m_hSteamLeaderboard != -1) ? true : false);
			} 
		}
		else if (bInit)
		{
			// Invalid API call
			bIsComplete = true;
			bWasSuccessful = false;
		}
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override
	{
		FOnlineAsyncTaskSteam::Finalize();

		// Mapping of players with stats
		TUniqueNetIdMap<int32> PlayersHaveStats;

		ISteamUserStats* SteamUserStatsPtr = SteamUserStats();
		for (int32 EntryIdx=0; EntryIdx < CallbackResults.m_cEntryCount; EntryIdx++)
		{
			LeaderboardEntry_t LeaderboardEntry;
			if (SteamUserStatsPtr->GetDownloadedLeaderboardEntry(CallbackResults.m_hSteamLeaderboardEntries, EntryIdx, &LeaderboardEntry, NULL, 0))
			{
				FUniqueNetIdSteamRef CurrentUser = FUniqueNetIdSteam::Create(LeaderboardEntry.m_steamIDUser);
				FOnlineStatsRow* UserRow = ReadObject->FindPlayerRecord(*CurrentUser);
				if (UserRow == NULL)
				{
					const FString NickName(UTF8_TO_TCHAR(SteamFriends()->GetFriendPersonaName(LeaderboardEntry.m_steamIDUser)));
					FUniqueNetIdRef UserId = FUniqueNetIdSteam::Create(LeaderboardEntry.m_steamIDUser);
					UserRow = new (ReadObject->Rows) FOnlineStatsRow(NickName, UserId);
				}

				// Only take the rank from here (stats grabs the actual ranked value)
				UserRow->Rank = LeaderboardEntry.m_nGlobalRank;
				PlayersHaveStats.Add(CurrentUser, 1);

				// Start up tasks to get stats. We don't do this like arbitrary user fetch does because we don't have the user list known at original query time.
				// This is fine to start now because we are still on the game thread.
				if (Type != RetrieveType::FetchUsers && Type != RetrieveType::FetchRankUser)
				{
					FOnlineAsyncTaskSteamRetrieveStats* NewStatsTask = new FOnlineAsyncTaskSteamRetrieveStats(Subsystem, *CurrentUser, 
						ReadObject, (EntryIdx + 1 == CallbackResults.m_cEntryCount));

					Subsystem->QueueAsyncTask(NewStatsTask);
				}
			}
		}

		// If we're looking for ranks around a user, we need to restart our query to get the actual entries.
		if (Type == RetrieveType::FetchRankUser)
		{
			FOnlineLeaderboardsSteamPtr Leaderboards = StaticCastSharedPtr<FOnlineLeaderboardsSteam>(Subsystem->GetLeaderboardsInterface());
			FOnlineStatsRow* UserRow = ReadObject->FindPlayerRecord(*Players[0]);

			// Only try to fetch stats for users that exist in the table.
			if (UserRow != NULL)
			{
				// This function will reset the current data we just got, this is fine 
				// because the ordering would have been broken anyways.
				Leaderboards->ReadLeaderboardsAroundRank(UserRow->Rank, Range, ReadObject);
				return;
			}
		}

		// Add placeholder stats for anyone who didn't show up on the leaderboard
		FVariantData EmptyData;
 		for (int32 UserIdx=0; UserIdx<Players.Num(); UserIdx++)
 		{
 			const FUniqueNetIdRef& CurrentUser = Players[UserIdx];
 			if (PlayersHaveStats.Find(CurrentUser) == NULL)
 			{
				FOnlineStatsRow* UserRow = ReadObject->FindPlayerRecord(*CurrentUser);
				if (UserRow == NULL)
				{
					const FUniqueNetIdSteam& SteamId = FUniqueNetIdSteam::Cast(*CurrentUser);
					const FString NickName(UTF8_TO_TCHAR(SteamFriends()->GetFriendPersonaName(SteamId)));
					UserRow = new (ReadObject->Rows) FOnlineStatsRow(NickName, SteamId.AsShared());

					// Don't process this for arbitrary list fetches as that automatically starts 
					// FOnlineAsyncTaskSteamRetrieveStats tasks
					if (Type != RetrieveType::FetchUsers)
					{
						// If the user is not on the leaderboard at all, push null stats for them.
						for (int32 StatIdx = 0; StatIdx < ReadObject->ColumnMetadata.Num(); StatIdx++)
						{
							const FColumnMetaData& ColumnMeta = ReadObject->ColumnMetadata[StatIdx];
							UserRow->Columns.Add(ColumnMeta.ColumnName, EmptyData);
						}
					}
				}

				// Only take the rank from here (stats grabs the actual ranked value)
				UserRow->Rank = -1;
 			}
 		}

		// If we have no data, we should mark that we're done processing here.
		// Normally FOnlineAsyncTaskSteamRetrieveStats would handle our readstats 
		// but if it never executes, then we should exit.
		if (CallbackResults.m_cEntryCount == 0 && Type != RetrieveType::FetchUsers)
		{
			ReadObject->ReadState = EOnlineAsyncTaskState::Done;
			bShouldTriggerDelegates = true;
		}
	}

	/**
	*	Async task is given a chance to trigger it's delegates
	*/
	virtual void TriggerDelegates() override
	{
		FOnlineAsyncTaskSteam::TriggerDelegates();

		// This is almost always false as FOnlineAsyncTaskSteamRetrieveStats will be the one to call the delegates for us.
		// This is only true if we have completely empty data.
		if (bShouldTriggerDelegates)
		{
			FOnlineLeaderboardsSteamPtr Leaderboards = StaticCastSharedPtr<FOnlineLeaderboardsSteam>(Subsystem->GetLeaderboardsInterface());
			Leaderboards->TriggerOnLeaderboardReadCompleteDelegates(bWasSuccessful);
		}
	}
};

/**
 *	Update a single leaderboard for the signed in user, Steam does not allow others to write for you
 */
class FOnlineAsyncTaskSteamUpdateLeaderboard : public FOnlineAsyncTaskSteam
{
private:

	/** Has this request been started */
	bool bInit;
	/** Name of leaderboard to update */
	FString LeaderboardName;
	/** Name of stat that will replace/update the existing value on the leaderboard */
	FString RatedStat;
	/** Score that will replace/update the existing value on the leaderboard */
	int32 NewScore;
	/** Method of update against the previous score */
	ELeaderboardUpdateMethod::Type UpdateMethod;
	/** Results returned from Steam backend */
	LeaderboardScoreUploaded_t CallbackResults;
	/** Since we can write multiple leaderboards with one call, indicate whether this is the last one */
	bool bShouldTriggerDelegates;

	FOnlineAsyncTaskSteamUpdateLeaderboard() :
		FOnlineAsyncTaskSteam(NULL, k_uAPICallInvalid),
		bInit(false),
		NewScore(0),
		UpdateMethod(ELeaderboardUpdateMethod::KeepBest),
		bShouldTriggerDelegates(false)
	{
	}

public:
	FOnlineAsyncTaskSteamUpdateLeaderboard(FOnlineSubsystemSteam* InSteamSubsystem, const FString& InLeaderboardName, const FString& InRatedStat, ELeaderboardUpdateMethod::Type InUpdateMethod, bool bInShouldTriggerDelegates) :
		FOnlineAsyncTaskSteam(InSteamSubsystem, k_uAPICallInvalid),
		bInit(false),
		LeaderboardName(InLeaderboardName),
		RatedStat(InRatedStat),
		NewScore(0),
		UpdateMethod(InUpdateMethod),
		bShouldTriggerDelegates(bInShouldTriggerDelegates)
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncTaskSteamUpdateLeaderboard bWasSuccessful: %d Leaderboard: %s Score: %d"), WasSuccessful(), *LeaderboardName, NewScore);
	}

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override
	{
		ISteamUtils* SteamUtilsPtr = SteamUtils();
		check(SteamUtilsPtr);

		if (!bInit)
		{
			// Poll for leaderboard handle
			SteamLeaderboard_t LeaderboardHandle = -1;
			{
				FOnlineLeaderboardsSteamPtr Leaderboards = StaticCastSharedPtr<FOnlineLeaderboardsSteam>(Subsystem->GetLeaderboardsInterface());
				FScopeLock ScopeLock(&Leaderboards->LeaderboardMetadataLock);
				FLeaderboardMetadataSteam* Leaderboard = Leaderboards->GetLeaderboardMetadata(LeaderboardName);
				if (Leaderboard)
				{
					LeaderboardHandle = Leaderboard->LeaderboardHandle;
				}
			}

			if (LeaderboardHandle != -1)
			{
				ISteamUserStats* SteamUserStatsPtr = SteamUserStats();
				check(SteamUserStatsPtr);

				ELeaderboardUploadScoreMethod UpdateMethodSteam; 
				switch(UpdateMethod)
				{
				case ELeaderboardUpdateMethod::Force:
					UpdateMethodSteam = k_ELeaderboardUploadScoreMethodForceUpdate;
					break;
				case ELeaderboardUpdateMethod::KeepBest:
				default:
					UpdateMethodSteam = k_ELeaderboardUploadScoreMethodKeepBest;
					break;
				}

				const FString RatedStatName = GetLeaderboardStatName(LeaderboardName, RatedStat);
				if (SteamUserStats()->GetStat(TCHAR_TO_UTF8(*RatedStatName), &NewScore))
				{
					CallbackHandle = SteamUserStatsPtr->UploadLeaderboardScore(LeaderboardHandle, UpdateMethodSteam, NewScore, NULL, 0);
				}
				
				bInit = true;
			}
		}

		if (CallbackHandle != k_uAPICallInvalid)
		{
			bool bFailedCall = false; 

			// Poll for completion status
			bIsComplete = SteamUtilsPtr->IsAPICallCompleted(CallbackHandle, &bFailedCall) ? true : false; 
			if (bIsComplete) 
			{ 
				bool bFailedResult;
				// Retrieve the callback data from the request
				bool bSuccessCallResult = SteamUtilsPtr->GetAPICallResult(CallbackHandle, &CallbackResults, sizeof(CallbackResults), CallbackResults.k_iCallback, &bFailedResult); 
				bWasSuccessful = (bSuccessCallResult ? true : false) &&
					(!bFailedCall ? true : false) &&
					(!bFailedResult ? true : false) &&
					((CallbackResults.m_bSuccess != 0) ? true : false);
			} 
		}
		else if (bInit)
		{
			// Invalid API call
			bIsComplete = true;
			bWasSuccessful = false;
		}
	}

	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override
	{
		FOnlineAsyncTaskSteam::TriggerDelegates();
		if (bShouldTriggerDelegates)
		{
			// TODO ONLINE - WriteOnlineStats delegate? Not flush.
		}
	}
};

/**
 *	Async task to store written Stats to the Steam backend
 *  Triggers an OnStatsStored callback
 */
class FOnlineAsyncTaskSteamStoreStats : public FOnlineAsyncTaskSteam
{
protected:

	/** Has this request been started */
	bool bInit;
	/** Name of session stats were written to (unused in Steam) */
	const FName SessionName;
	/** User this store is for */
	const FUniqueNetIdSteamRef UserId;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamStoreStats() :
		FOnlineAsyncTaskSteam(NULL, k_uAPICallInvalid),
		bInit(false),
		SessionName(NAME_None),
		UserId(FUniqueNetIdSteam::EmptyId())
	{
	}

	/** Internal function to allow write state tracking */
	virtual void OperationStarted()
	{
		/** No op in base */
	}

	/** Internal function to allow write state tracking */
	virtual void OperationFailed()
	{
		/** No op in base */
	}

	/** Internal function to allow write state tracking */
	virtual void OperationSucceeded()
	{
		/** No op in base */
	}

public:
	FOnlineAsyncTaskSteamStoreStats(FOnlineSubsystemSteam* InSteamSubsystem, const FName& InSessionName, const FUniqueNetIdSteam& InUserId) :
		FOnlineAsyncTaskSteam(InSteamSubsystem, k_uAPICallInvalid),
		bInit(false),
		SessionName(InSessionName),
		UserId(InUserId.AsShared())
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncTaskSteamStoreStats SessionName: %s bWasSuccessful: %d"), *SessionName.ToString(), WasSuccessful());
	}

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override
	{
		FOnlineLeaderboardsSteamPtr Leaderboards = StaticCastSharedPtr<FOnlineLeaderboardsSteam>(Subsystem->GetLeaderboardsInterface());

		if (!bInit)
		{
			OperationStarted();
			bInit = true;
			Leaderboards->UserStatsStoreStatsFinishedDelegate.BindRaw(this, &FOnlineAsyncTaskSteamStoreStats::OnUserStatsStoreStatsFinished);
			if (!SteamUserStats()->StoreStats())
			{
				OnUserStatsStoreStatsFinished(EOnlineAsyncTaskState::Failed);
				return;
			}
		}
	}

	void OnUserStatsStoreStatsFinished(EOnlineAsyncTaskState::Type State)
	{
		FOnlineLeaderboardsSteamPtr Leaderboards = StaticCastSharedPtr<FOnlineLeaderboardsSteam>(Subsystem->GetLeaderboardsInterface());
		Leaderboards->UserStatsStoreStatsFinishedDelegate.Unbind();

		bIsComplete = true;
		bWasSuccessful = State == EOnlineAsyncTaskState::Done;

		if (bWasSuccessful)
		{
			OperationSucceeded();
		}
		else
		{
			OperationFailed();
		}
	}
};

/**
 *	Async task to store written Stats to the Steam backend
 *  Triggers an OnStatsStored callback
 */
class FOnlineAsyncTaskSteamFlushLeaderboards : public FOnlineAsyncTaskSteamStoreStats
{
private:

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamFlushLeaderboards() :
	   FOnlineAsyncTaskSteamStoreStats()
	{
	}

public:
	FOnlineAsyncTaskSteamFlushLeaderboards(FOnlineSubsystemSteam* InSteamSubsystem, const FName& InSessionName, const FUniqueNetIdSteam& InUserId) :
		FOnlineAsyncTaskSteamStoreStats(InSteamSubsystem, InSessionName, InUserId)
	{
	}

	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override 
	{
		FOnlineAsyncTaskSteam::TriggerDelegates();
		
		FOnlineLeaderboardsSteamPtr Leaderboards = StaticCastSharedPtr<FOnlineLeaderboardsSteam>(Subsystem->GetLeaderboardsInterface());
		Leaderboards->TriggerOnLeaderboardFlushCompleteDelegates(SessionName, bWasSuccessful);
	}
};

/**
 *	Async task to store written Stats to the Steam backend
 *  Triggers an OnStatsStored callback
 */
class FOnlineAsyncTaskSteamWriteAchievements : public FOnlineAsyncTaskSteamStoreStats
{
private:

	/** Reference to write object for state tracking */
	FOnlineAchievementsWritePtr WriteObject;

	/** Delegate to call when the write finishes */
	FOnAchievementsWrittenDelegate OnWriteFinishedDelegate;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamWriteAchievements() :
		FOnlineAsyncTaskSteamStoreStats(),
		WriteObject(NULL)
	{
	}

	/** Internal function to allow write state tracking */
	virtual void OperationStarted() override
	{
		check(WriteObject.IsValid());
		WriteObject->WriteState = EOnlineAsyncTaskState::InProgress;
	}

	/** Internal function to allow write state tracking */
	virtual void OperationFailed() override
	{
		check(WriteObject.IsValid());
		WriteObject->WriteState = EOnlineAsyncTaskState::Failed;
	}

	/** Internal function to allow write state tracking */
	virtual void OperationSucceeded() override
	{
		check(WriteObject.IsValid());
		WriteObject->WriteState = EOnlineAsyncTaskState::Done;
	}

public:
	FOnlineAsyncTaskSteamWriteAchievements(FOnlineSubsystemSteam* InSteamSubsystem, const FUniqueNetIdSteam& InUserId, FOnlineAchievementsWriteRef& InWriteObject, const FOnAchievementsWrittenDelegate& InOnWriteFinishedDelegate)
		: FOnlineAsyncTaskSteamStoreStats(InSteamSubsystem, TEXT("Unused"), InUserId)
		, WriteObject(InWriteObject)
		, OnWriteFinishedDelegate(InOnWriteFinishedDelegate)
	{
	}

	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override 
	{
		FOnlineAsyncTaskSteam::TriggerDelegates();
		
		FOnlineAchievementsSteamPtr Achievements = StaticCastSharedPtr<FOnlineAchievementsSteam>(Subsystem->GetAchievementsInterface());
		Achievements->OnWriteAchievementsComplete(*UserId, bWasSuccessful, WriteObject, OnWriteFinishedDelegate);
	}
};

bool FOnlineLeaderboardsSteam::ReadLeaderboards(const TArray< FUniqueNetIdRef >& Players, FOnlineLeaderboardReadRef& ReadObject)
{
	ReadObject->ReadState = EOnlineAsyncTaskState::InProgress;

	// Clear out any existing data
	ReadObject->Rows.Empty();

	// Will retrieve the leaderboard, making async calls as appropriate
	FindLeaderboard(ReadObject->LeaderboardName);
	
	// Retrieve the leaderboard data
	FOnlineAsyncTaskSteamRetrieveLeaderboardEntries* NewLeaderboardTask = new FOnlineAsyncTaskSteamRetrieveLeaderboardEntries(SteamSubsystem, Players, ReadObject);
	SteamSubsystem->QueueAsyncTask(NewLeaderboardTask);

	// Retrieve the stats related to this leaderboard
	int32 NumPlayers = Players.Num();
	for (int32 UserIdx=0; UserIdx < NumPlayers; UserIdx++)
	{
		bool bLastPlayer = (UserIdx == NumPlayers-1) ? true : false;
		const FUniqueNetIdSteam& UserId = FUniqueNetIdSteam::Cast(*Players[UserIdx]);
		FOnlineAsyncTaskSteamRetrieveStats* NewStatsTask = new FOnlineAsyncTaskSteamRetrieveStats(SteamSubsystem, UserId, ReadObject, bLastPlayer);
		SteamSubsystem->QueueAsyncTask(NewStatsTask);
	}

	return true;
}

bool FOnlineLeaderboardsSteam::ReadLeaderboardsAroundRank(int32 Rank, uint32 Range, FOnlineLeaderboardReadRef& ReadObject)
{
	ReadObject->ReadState = EOnlineAsyncTaskState::InProgress;

	// Clear out any existing data
	ReadObject->Rows.Empty();

	// Will retrieve the leaderboard, making async calls as appropriate
	FindLeaderboard(ReadObject->LeaderboardName);

	FOnlineAsyncTaskSteamRetrieveLeaderboardEntries* NewLeaderboardTask = new FOnlineAsyncTaskSteamRetrieveLeaderboardEntries(SteamSubsystem, Rank, Range, ReadObject);
	SteamSubsystem->QueueAsyncTask(NewLeaderboardTask);

	return true;
}
bool FOnlineLeaderboardsSteam::ReadLeaderboardsAroundUser(FUniqueNetIdRef Player, uint32 Range, FOnlineLeaderboardReadRef& ReadObject)
{
	ReadObject->ReadState = EOnlineAsyncTaskState::InProgress;

	// Clear out any existing data
	ReadObject->Rows.Empty();

	// Will retrieve the leaderboard, making async calls as appropriate
	FindLeaderboard(ReadObject->LeaderboardName);

	FOnlineAsyncTaskSteamRetrieveLeaderboardEntries* NewLeaderboardTask = new FOnlineAsyncTaskSteamRetrieveLeaderboardEntries(SteamSubsystem, Player, Range, ReadObject);

	SteamSubsystem->QueueAsyncTask(NewLeaderboardTask);

	return true;
}

bool FOnlineLeaderboardsSteam::ReadLeaderboardsForFriends(int32 LocalUserNum, FOnlineLeaderboardReadRef& ReadObject)
{
	ReadObject->ReadState = EOnlineAsyncTaskState::InProgress;

	// Clear out any existing data
	ReadObject->Rows.Empty();

	// Will retrieve the leaderboard, making async calls as appropriate
	FindLeaderboard(ReadObject->LeaderboardName);

	FOnlineAsyncTaskSteamRetrieveLeaderboardEntries* NewLeaderboardTask = new FOnlineAsyncTaskSteamRetrieveLeaderboardEntries(SteamSubsystem, ReadObject);
	SteamSubsystem->QueueAsyncTask(NewLeaderboardTask);

	return true;
}

void FOnlineLeaderboardsSteam::QueryAchievementsInternal(const FUniqueNetIdSteam& UserId, const FOnQueryAchievementsCompleteDelegate& AchievementDelegate)
{
	FOnlineAsyncTaskSteamGetAchievements* NewStatsTask = new FOnlineAsyncTaskSteamGetAchievements(SteamSubsystem, UserId, AchievementDelegate);

	SteamSubsystem->QueueAsyncTask(NewStatsTask);
}


void FOnlineLeaderboardsSteam::FreeStats(FOnlineLeaderboardRead& ReadObject)
{
	// NOOP
}

bool FOnlineLeaderboardsSteam::WriteLeaderboards(const FName& SessionName, const FUniqueNetId& Player, FOnlineLeaderboardWrite& WriteObject)
{
	bool bWasSuccessful = true;

	// Find or create handles to all requested leaderboards (async)
	int32 NumLeaderboards = WriteObject.LeaderboardNames.Num();
	for (int32 LeaderboardIdx = 0; LeaderboardIdx < NumLeaderboards; LeaderboardIdx++)
	{
		// Will create or retrieve the leaderboards, triggering async calls as appropriate
		CreateLeaderboard(WriteObject.LeaderboardNames[LeaderboardIdx], WriteObject.SortMethod, WriteObject.DisplayFormat);
	}

	// Update stats columns associated with the leaderboards (before actual leaderboard update so we can retrieve the updated stat)
	FStatPropertyArray LeaderboardStats;
	for (int32 LeaderboardIdx = 0; LeaderboardIdx < NumLeaderboards; LeaderboardIdx++)
	{
		for (FStatPropertyArray::TConstIterator It(WriteObject.Properties); It; ++It)
		{
			const FVariantData& Stat = It.Value();
			const FString LeaderboardStatName = GetLeaderboardStatName(WriteObject.LeaderboardNames[LeaderboardIdx], It.Key());
			LeaderboardStats.Add(LeaderboardStatName, Stat);
		}
	}

	const FUniqueNetIdSteam& UserId = FUniqueNetIdSteam::Cast(Player);
	FOnlineAsyncTaskSteamUpdateStats* NewUpdateStatsTask = new FOnlineAsyncTaskSteamUpdateStats(SteamSubsystem, UserId, LeaderboardStats);
	SteamSubsystem->QueueAsyncTask(NewUpdateStatsTask);

	// Update all leaderboards (async)
	for (int32 LeaderboardIdx = 0; LeaderboardIdx < NumLeaderboards; LeaderboardIdx++)
	{
		bool bLastLeaderboard = (LeaderboardIdx == NumLeaderboards - 1) ? true : false;

		FOnlineAsyncTaskSteamUpdateLeaderboard* NewUpdateLeaderboardTask = 
			new FOnlineAsyncTaskSteamUpdateLeaderboard(SteamSubsystem, WriteObject.LeaderboardNames[LeaderboardIdx], WriteObject.RatedStat, WriteObject.UpdateMethod, bLastLeaderboard);
		SteamSubsystem->QueueAsyncTask(NewUpdateLeaderboardTask);
	}

	return bWasSuccessful;
}

void FOnlineLeaderboardsSteam::WriteAchievementsInternal(const FUniqueNetIdSteam& UserId, FOnlineAchievementsWriteRef& WriteObject, const FOnAchievementsWrittenDelegate& OnWriteFinishedDelegate)
{
	FOnlineAsyncTaskSteamWriteAchievements* NewTask = new FOnlineAsyncTaskSteamWriteAchievements(SteamSubsystem, UserId, WriteObject, OnWriteFinishedDelegate);
	SteamSubsystem->QueueAsyncTask(NewTask);
}

bool FOnlineLeaderboardsSteam::FlushLeaderboards(const FName& SessionName)
{
	const FUniqueNetIdSteamRef UserId = FUniqueNetIdSteam::Create(SteamUser()->GetSteamID());
	FOnlineAsyncTaskSteamFlushLeaderboards* NewTask = new FOnlineAsyncTaskSteamFlushLeaderboards(SteamSubsystem, SessionName, *UserId);
	SteamSubsystem->QueueAsyncTask(NewTask);
	return true;
}

bool FOnlineLeaderboardsSteam::WriteOnlinePlayerRatings(const FName& SessionName, int32 LeaderboardId, const TArray<FOnlinePlayerScore>& PlayerScores)
{
	return false;
}

FLeaderboardMetadataSteam* FOnlineLeaderboardsSteam::GetLeaderboardMetadata(const FString& LeaderboardName)
{
	FScopeLock ScopeLock(&LeaderboardMetadataLock);
	for (int32 LeaderboardIdx = 0; LeaderboardIdx < Leaderboards.Num(); LeaderboardIdx++)
	{
		const FLeaderboardMetadataSteam& Leaderboard = Leaderboards[LeaderboardIdx];
		if (Leaderboard.LeaderboardName == LeaderboardName)
		{
			return &Leaderboards[LeaderboardIdx];
		}
	}

	return NULL;
}

void FOnlineLeaderboardsSteam::CreateLeaderboard(const FString& LeaderboardName, ELeaderboardSort::Type SortMethod, ELeaderboardFormat::Type DisplayFormat)
{
	FScopeLock ScopeLock(&LeaderboardMetadataLock);
	FLeaderboardMetadataSteam* LeaderboardMetadata = GetLeaderboardMetadata(LeaderboardName);

	// Don't allow multiple attempts to create a leaderboard unless it's actually failed before
	bool bPrevAttemptFailed = (LeaderboardMetadata != NULL && LeaderboardMetadata->LeaderboardHandle == -1 &&
								(LeaderboardMetadata->AsyncState == EOnlineAsyncTaskState::Done ||
								 LeaderboardMetadata->AsyncState == EOnlineAsyncTaskState::Failed));

	if (LeaderboardMetadata == NULL || bPrevAttemptFailed)
	{
		FLeaderboardMetadataSteam* NewLeaderboard = new (Leaderboards) FLeaderboardMetadataSteam(LeaderboardName, SortMethod, DisplayFormat);
		NewLeaderboard->AsyncState = EOnlineAsyncTaskState::InProgress;
		SteamSubsystem->QueueAsyncTask(new FOnlineAsyncTaskSteamRetrieveLeaderboard(SteamSubsystem, LeaderboardName, SortMethod, DisplayFormat));
	}
	// else request already in flight or already found
}

void FOnlineLeaderboardsSteam::FindLeaderboard(const FString& LeaderboardName)
{
	FScopeLock ScopeLock(&LeaderboardMetadataLock);
	FLeaderboardMetadataSteam* LeaderboardMetadata = GetLeaderboardMetadata(LeaderboardName);

	// Don't allow multiple attempts to find a leaderboard unless it's actually failed before
	bool bPrevAttemptFailed = (LeaderboardMetadata != NULL && LeaderboardMetadata->LeaderboardHandle == -1 &&
								(LeaderboardMetadata->AsyncState == EOnlineAsyncTaskState::Done ||
								 LeaderboardMetadata->AsyncState == EOnlineAsyncTaskState::Failed));

	if (LeaderboardMetadata == NULL || bPrevAttemptFailed)
	{	
		// No current find or create in flight
		FLeaderboardMetadataSteam* NewLeaderboard = new (Leaderboards) FLeaderboardMetadataSteam(LeaderboardName);
		NewLeaderboard->AsyncState = EOnlineAsyncTaskState::InProgress;
		SteamSubsystem->QueueAsyncTask(new FOnlineAsyncTaskSteamRetrieveLeaderboard(SteamSubsystem, LeaderboardName));
	}
	// else request already in flight or already found
}