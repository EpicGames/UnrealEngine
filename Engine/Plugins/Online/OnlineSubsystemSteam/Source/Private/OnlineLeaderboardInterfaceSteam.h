// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemPackage.h"
#include "OnlineSubsystemSteamTypes.h"
#include "Interfaces/OnlineLeaderboardInterface.h"
#include "Interfaces/OnlineAchievementsInterface.h"

class FOnlineSubsystemSteam;

/**
 *	Keeps track of the download state of any stats requests/unloads
 * Requests typically initiate in game, but Steam may unload stats at any time
 */
struct FUserStatsStateSteam
{
private:
	/** Hidden on purpose */
	FUserStatsStateSteam() = delete;

public:
	/** User Id */
	FUniqueNetIdSteamRef UserId;
	/** Current stats state for this user */
	EOnlineAsyncTaskState::Type StatsState;

	FUserStatsStateSteam(const FUniqueNetIdSteam& InUserId, EOnlineAsyncTaskState::Type InState) :
		UserId(InUserId.AsShared()),
		StatsState(InState)
	{
	}
};

DECLARE_DELEGATE_OneParam(FOnSteamUserStatsStoreStatsFinished, EOnlineAsyncTaskState::Type);

/**
 * Interface definition for the online services leaderboard services 
 */
class FOnlineLeaderboardsSteam : public IOnlineLeaderboards
{
private:
	/** Reference to the main Steam subsystem */
	class FOnlineSubsystemSteam* SteamSubsystem;

	/** Array of known leaderboards (may be more that haven't been requested from) */
	TArray<FLeaderboardMetadataSteam> Leaderboards;

	FOnlineLeaderboardsSteam() : 
		SteamSubsystem(NULL)
	{
	}

PACKAGE_SCOPE:

	/** Critical section for thread safe operation of the leaderboard metadata */
	FCriticalSection LeaderboardMetadataLock;
	/** Critical section for thread safe operation of the stats stored state */
	FCriticalSection UserStatsStoredLock;
	/** Called when the event from SteamUserStats()->StoreStats() finishing is triggered */
	FOnSteamUserStatsStoreStatsFinished UserStatsStoreStatsFinishedDelegate;

	FOnlineLeaderboardsSteam(FOnlineSubsystemSteam* InSteamSubsystem) :
		SteamSubsystem(InSteamSubsystem)
	{
	}

	/**
	 *	Get the leaderboard metadata for a given leaderboard
	 * If the data doesn't exist, the game hasn't asked to Create or Find the leaderboard below
	 * 
	 * @param LeaderboardName name of leaderboard to get information for
	 * @return leaderboard metadata if it exists, else NULL
	 */
	FLeaderboardMetadataSteam* GetLeaderboardMetadata(const FString& LeaderboardName);

	/**
	 *	Start an async task to create a leaderboard with the Steam backend
	 * If the leaderboard already exists, the leaderboard data will still be retrieved
	 * @param LeaderboardName name of leaderboard to create
	 * @param SortMethod method the leaderboard scores will be sorted, ignored if leaderboard exists
	 * @param DisplayFormat type of data the leaderboard represents, ignored if leaderboard exists
	 */
	void CreateLeaderboard(const FString& LeaderboardName, ELeaderboardSort::Type SortMethod, ELeaderboardFormat::Type DisplayFormat);

	/**
	 *	Start an async task to find a leaderboard with the Steam backend
	 * If the leaderboard doesn't exist, a warning will be generated
	 * @param LeaderboardName name of leaderboard to create
	 */
	void FindLeaderboard(const FString& LeaderboardName);

	/**
	 *	Get the received stats state for a given user (game thread only)
	 * 
	 * @param User user to check if stats have been received
	 * @return State of the stats download for a given user, or NotStarted if user isn't found
	 */
	EOnlineAsyncTaskState::Type GetUserStatsState(const FUniqueNetIdSteam& UserId);

	/**
	 *	Set the received stats state for a given user (game thread only)
	 * 
	 * @param User user to set state of stats received
	 * @param NewState state of the stats download for a given user
	 */
	void SetUserStatsState(const FUniqueNetIdSteam& UserId, EOnlineAsyncTaskState::Type NewState);

	/**
	 * Commits any changes in the online stats cache to the permanent storage (internal helper for FOnlineAchievementsSteam::WriteAchievements)
	 *
	 * @param UserId user to set state of stats stored
	 * @param WriteObject object to track the state of stats
	 */
	void WriteAchievementsInternal(const FUniqueNetIdSteam& UserId, FOnlineAchievementsWriteRef& WriteObject, const FOnAchievementsWrittenDelegate& OnWriteFinishedDelegate);

	/**
	 * Reads achievements for user (internal helper for FOnlineAchievementsSteam::QueryAchievements)
	 *
	 * @param UserId user to set state of stats stored
	 * @param bTriggerDescriptionsDelegatesInstead - async task will normally trigger OnAchievementsRead delegates, this bool tells it to trigger OnAchievementDescriptionsRead instead
	 */
	void QueryAchievementsInternal(const FUniqueNetIdSteam& UserId, const FOnQueryAchievementsCompleteDelegate& AchievementDelegate);

public:

	virtual ~FOnlineLeaderboardsSteam() {};

	// IOnlineLeaderboards
	virtual bool ReadLeaderboards(const TArray< FUniqueNetIdRef >& Players, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual bool ReadLeaderboardsForFriends(int32 LocalUserNum, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual bool ReadLeaderboardsAroundRank(int32 Rank, uint32 Range, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual bool ReadLeaderboardsAroundUser(FUniqueNetIdRef Player, uint32 Range, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual void FreeStats(FOnlineLeaderboardRead& ReadObject) override;
	virtual bool WriteLeaderboards(const FName& SessionName, const FUniqueNetId& Player, FOnlineLeaderboardWrite& WriteObject) override;
	virtual bool FlushLeaderboards(const FName& SessionName) override;
	virtual bool WriteOnlinePlayerRatings(const FName& SessionName, int32 LeaderboardId, const TArray<FOnlinePlayerScore>& PlayerScores) override;
};

typedef TSharedPtr<FOnlineLeaderboardsSteam, ESPMode::ThreadSafe> FOnlineLeaderboardsSteamPtr;

