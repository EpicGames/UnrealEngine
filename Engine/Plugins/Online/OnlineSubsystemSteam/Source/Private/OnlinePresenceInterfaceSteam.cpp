// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlinePresenceInterfaceSteam.h"
#include "OnlineSubsystemSteam.h"
#include "OnlineSessionInterfaceSteam.h"
#include <steam/isteamutils.h>
#include "Misc/ConfigCacheIni.h"

/** The default key that will contain the launch parameters for joining the game */
const FString DefaultSteamConnectionKey = TEXT("connect");

/** The default key that will update presence text in the platform's UI */
const FString DefaultSteamPresenceKey = TEXT("steam_display");

/** The default key that will group players in the platform's UI */
const char* DefaultSteamPlayerGroupingKey = "steam_player_group";

/** The default key that specifies the amount of players are in the party for the platform's UI */
const char* DefaultSteamPlayerGroupSizeKey = "steam_player_group_size";

void FOnlineUserPresenceSteam::Update(const FUniqueNetIdSteam& FriendId)
{
	Reset();
	ISteamFriends* SteamFriendPtr = SteamFriends();
	if (SteamFriendPtr == nullptr || SteamUtils() == nullptr)
	{
		return;
	}

	FriendGameInfo_t FriendInfo;
	bIsAFriend = (SteamFriendPtr->GetFriendRelationship(FriendId) == k_EFriendRelationshipFriend);
	bIsOnline = true;

	// Set Online State
	switch (SteamFriendPtr->GetFriendPersonaState(FriendId))
	{
		default:
		case k_EPersonaStateOffline:
			bIsOnline = false;
			Status.State = EOnlinePresenceState::Offline;
			break;
		case k_EPersonaStateBusy:
			Status.State = EOnlinePresenceState::DoNotDisturb;
			break;
		case k_EPersonaStateAway:
			Status.State = EOnlinePresenceState::Away;
			break;
		case k_EPersonaStateSnooze:
			Status.State = EOnlinePresenceState::ExtendedAway;
			break;
		case k_EPersonaStateOnline:
		case k_EPersonaStateLookingToTrade:
		case k_EPersonaStateLookingToPlay:
			Status.State = EOnlinePresenceState::Online;
			break;
	}

	// Get Game information, if this returns true, we are in a game
	if (SteamFriendPtr->GetFriendGamePlayed(FriendId, &FriendInfo))
	{
		bIsPlaying = true;
		SessionId = FUniqueNetIdSteam::Create(FriendInfo.m_steamIDLobby); 
		bIsPlayingThisGame = (FriendInfo.m_gameID.AppID() == SteamUtils()->GetAppID());
	}
	else
	{
		bIsPlaying = false;
		SessionId = nullptr;
		bIsPlayingThisGame = false;
	}

	bIsJoinable = false;
	bHasVoiceSupport = false;

	// Processing presence
	for (int32 RPIdx = 0; RPIdx < SteamFriendPtr->GetFriendRichPresenceKeyCount(FriendId); ++RPIdx)
	{
		FString Key = SteamFriendPtr->GetFriendRichPresenceKeyByIndex(FriendId, RPIdx);
		FString Value = SteamFriendPtr->GetFriendRichPresence(FriendId, TCHAR_TO_UTF8(*Key));

		// This key is one of the two magic keys Steam defines. If we have it, that means the session is joinable.
		if (Key == DefaultSteamConnectionKey)
		{
			bIsJoinable = true;
			continue;
		}

		// This key has it's own entry, there's no need to push it to the property field
		if (Key == DefaultSteamPresenceKey)
		{
			Status.StatusStr = Value;
			continue;
		}

		// Determine if the user has voice support
		if (Key == "HasVoice" && Value == "true")
		{
			bHasVoiceSupport = true;
			continue;
		}

		FVariantData PropertyData;
		PropertyData.SetValue(Value);
		Status.Properties.Add(Key, PropertyData);
	}
}


FOnlinePresenceSteam::FOnlinePresenceSteam(class FOnlineSubsystemSteam* InSubsystem) :
	SteamFriendsPtr(SteamFriends()),
	SteamSubsystem(InSubsystem)
	
{
}

FOnlinePresenceSteam::FOnlinePresenceSteam() : 
	SteamFriendsPtr(nullptr),
	SteamSubsystem(nullptr)
{
}

void FOnlinePresenceSteam::SetPresence(const FUniqueNetId& User, const FOnlineUserPresenceStatus& Status, const FOnPresenceTaskCompleteDelegate& Delegate)
{
	if (SteamFriendsPtr == nullptr || SteamSubsystem == nullptr)
	{
		UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Steam friends is null, cannot set presence!"));
		Delegate.ExecuteIfBound(User, false);
		return;
	}

	// Calls to this function will return immediately. All of these calls are supposed to be constant time, non-blocking
	SteamFriendsPtr->ClearRichPresence();

	// Check we won't overflow from having too much data
	if (Status.Properties.Num() > k_cchMaxRichPresenceKeys)
	{
		// This doesn't account for the rich presence status and the connection information.
		UE_LOG_ONLINE_PRESENCE(Error, TEXT("Number of presence properties (%d) exceeds maximum keys allowed!"), Status.Properties.Num());
		Delegate.ExecuteIfBound(User, false);
		return;
	}

	// Push presence string
	// Steam status keys must begin with #, so insert it if not present
	FString StatusString = Status.StatusStr;
	if(!StatusString.StartsWith(TEXT("#")))
	{
		StatusString = FString::Printf(TEXT("#%s"), *StatusString);
	}

	if (!SteamFriendsPtr->SetRichPresence(TCHAR_TO_UTF8(*DefaultSteamPresenceKey), TCHAR_TO_UTF8(*StatusString)))
	{
		if (Status.StatusStr.Len() >= k_cchMaxRichPresenceValueLength)
		{
			UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Cannot push rich presence status to steam, string is too long (%d)"), StatusString.Len());
		}
		else
		{
			UE_LOG_ONLINE_PRESENCE(Warning, TEXT("An unknown error occurred when trying to push rich presence status to steam!"));
		}
	}

	// Pull session information if it exists, otherwise don't do it.
	FOnlineSessionSteamPtr SessionInterface = StaticCastSharedPtr<FOnlineSessionSteam>(SteamSubsystem->GetSessionInterface());
	if (SessionInterface.IsValid())
	{
		FNamedOnlineSession* CurrentSession = SessionInterface->GetNamedSession(NAME_GameSession);
		if (CurrentSession != NULL && CurrentSession->SessionSettings.bAllowJoinViaPresence)
		{
			FString SteamConnectString = SessionInterface->GetSteamConnectionString(NAME_GameSession);
			if (!SteamConnectString.IsEmpty() && !SteamFriendsPtr->SetRichPresence(TCHAR_TO_UTF8(*DefaultSteamConnectionKey), TCHAR_TO_UTF8(*SteamConnectString)))
			{
				UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Could not push the connection information to Steam"));
			}
		}

		bool bDisableSteamAdvancedPresence = false;
		GConfig->GetBool(TEXT("OnlineSubsystemSteam"), TEXT("DisableAdvancedPresence"), bDisableSteamAdvancedPresence, GEngineIni);
		if (CurrentSession != NULL && CurrentSession->SessionSettings.bUsesPresence && !bDisableSteamAdvancedPresence)
		{
			FString SteamPlayerGroupingKeyString = CurrentSession->GetSessionIdStr();
			if (!SteamPlayerGroupingKeyString.IsEmpty() && !SteamFriendsPtr->SetRichPresence(DefaultSteamPlayerGroupingKey, TCHAR_TO_UTF8(*SteamPlayerGroupingKeyString)))
			{
				UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Could not push the player grouping key information to Steam"));
			}

			FString SteamPlayerGroupSizeString = FString::FromInt(CurrentSession->RegisteredPlayers.Num());
			if (!SteamPlayerGroupSizeString.IsEmpty() && !SteamFriendsPtr->SetRichPresence(DefaultSteamPlayerGroupSizeKey, TCHAR_TO_UTF8(*SteamPlayerGroupSizeString)))
			{
				UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Could not push the player group size information to Steam"));
			}
		}
	}

	// Push extra presence properties
	for (FPresenceProperties::TConstIterator Itr(Status.Properties); Itr; ++Itr)
	{
		const FString& Key = Itr.Key();
		const FString& Value = Itr.Value().ToString();
		if (!SteamFriendsPtr->SetRichPresence(TCHAR_TO_UTF8(*Key), TCHAR_TO_UTF8(*Value)))
		{
			if (Key.Len() >= k_cchMaxRichPresenceKeyLength || Key.IsEmpty())
			{
				UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Steam presence key %s is either empty or over the max length of a key"), *Key);
			}
			else if(Value.Len() >= k_cchMaxRichPresenceValueLength)
			{
				UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Steam presence value for key %s (%d) is over the max size allowed"), *Key, Value.Len());
			}
			else
			{
				// Misc errors, typically this means you have too many keys pushed (~20)
				// If you hit this warning, remember to account for connect and status as keys
				UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Could not push presence key %s to steam!"), *Key);
			}
		}
	}

	// Force a cache update. Since this is a local user, there will be no outbound call, and this will force
	// a copy of the player's details to exist in cache (this also fixes the online/offline statuses) if 
	// it doesn't already exist.
	QueryPresence(User, Delegate);
}

void FOnlinePresenceSteam::QueryPresence(const FUniqueNetId& User, const FOnPresenceTaskCompleteDelegate& Delegate)
{
	if (SteamFriendsPtr == nullptr || SteamSubsystem == nullptr)
	{
		UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Steam friends is null, cannot fetch presence!"));
		Delegate.ExecuteIfBound(User, false);
		return;
	}

	const FUniqueNetIdSteam& SteamId = FUniqueNetIdSteam::Cast(User);
	if (!SteamId.IsValid())
	{
		UE_LOG_ONLINE_PRESENCE(Warning, TEXT("User id %s is not valid, cannot query presence"), *SteamId.ToString());
		Delegate.ExecuteIfBound(User, false);
		return;
	}

	TSharedRef<FOnlineUserPresenceSteam>* FoundEntry = CachedPresence.Find(SteamId.AsShared());
	
	// If we can't find it, create it.
	if (FoundEntry == nullptr)
	{
		TSharedRef<FOnlineUserPresenceSteam> Presence(new FOnlineUserPresenceSteam());
		FoundEntry = &CachedPresence.Add(SteamId.AsShared(), Presence);
	}

	// We cannot just grab non-friend information, and have to request Steam to fetch it for us.
	// As such, we cache the delegates for later and wait for Steam to let us know when the data is here
	if (SteamFriendsPtr->GetFriendRelationship(SteamId) != k_EFriendRelationshipFriend && !SteamSubsystem->IsLocalPlayer(User))
	{
		// Cache out the delegate for now, we'll need it later.
		DelayedPresenceDelegates.Add(SteamId.AsShared(), MakeShared<const FOnPresenceTaskCompleteDelegate>(Delegate));
		SteamFriendsPtr->RequestFriendRichPresence(SteamId);
		return;
	}

	// If the user is already on your friends, then all friend fetching calls are constant time calls
	// As we have direct access to the Steam client's friend data that it periodically updates
	FoundEntry->Get().Update(SteamId);
	Delegate.ExecuteIfBound(User, true);
}

void FOnlinePresenceSteam::UpdatePresenceForUser(const FUniqueNetId& User)
{
	// Filter out any local users from this callback.
	if (SteamFriendsPtr == nullptr || SteamSubsystem == nullptr || SteamSubsystem->IsLocalPlayer(User))
	{
		return;
	}

	const FUniqueNetIdSteam& SteamId = FUniqueNetIdSteam::Cast(User);
	TSharedRef<FOnlineUserPresenceSteam>* FoundEntry = CachedPresence.Find(SteamId.AsShared());

	// Create entries if we don't have them. 
	// Normally we would assume you have this information, however we get update callbacks on application start
	// that could potentially lead to us not having entries for this user.
	if (FoundEntry == nullptr)
	{
		TSharedRef<FOnlineUserPresenceSteam> Presence(new FOnlineUserPresenceSteam());
		FoundEntry = &CachedPresence.Add(SteamId.AsShared(), Presence);
	}

	FoundEntry->Get().Update(SteamId);

	// If this user was not a friend at the time of the QueryPresence call, they likely have a delegate registered
	// Find it, and then call it, removing the old entry.
	TSharedRef<const FOnPresenceTaskCompleteDelegate>* DelayedDelegate = DelayedPresenceDelegates.Find(SteamId.AsShared());
	if (DelayedDelegate != nullptr)
	{
		DelayedDelegate->Get().ExecuteIfBound(User, true);
		DelayedPresenceDelegates.Remove(SteamId.AsShared());
	}
	else
	{
		// Otherwise this was an actual change, and not the result of a query
		// So trigger the delegates
		TArray<TSharedRef<FOnlineUserPresence> > PresenceArray;
		PresenceArray.Add(*FoundEntry);

		TriggerOnPresenceArrayUpdatedDelegates(User, PresenceArray);
		TriggerOnPresenceReceivedDelegates(User, *FoundEntry);
	}
}

EOnlineCachedResult::Type FOnlinePresenceSteam::GetCachedPresence(const FUniqueNetId& User, TSharedPtr<FOnlineUserPresence>& OutPresence)
{
	TSharedRef<FOnlineUserPresenceSteam>* Found = CachedPresence.Find(User.AsShared());
	if (Found == nullptr)
	{
		return EOnlineCachedResult::NotFound;
	}

	OutPresence = Found->ToSharedPtr();
	return EOnlineCachedResult::Success;
}

EOnlineCachedResult::Type FOnlinePresenceSteam::GetCachedPresenceForApp(const FUniqueNetId& LocalUserId, const FUniqueNetId& User, const FString& AppId, TSharedPtr<FOnlineUserPresence>& OutPresence)
{
	// Cannot get detailed presence information for anyone but our own app
	if (SteamUtils() != nullptr && SteamUtils()->GetAppID() == FCString::Atoi(*AppId))
	{
		return GetCachedPresence(User, OutPresence);
	}

	return EOnlineCachedResult::NotFound;
}
