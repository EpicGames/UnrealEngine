// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlinePresenceInterface.h"
#include "OnlineSubsystemSteamPackage.h"

class FUniqueNetIdSteam;

class FOnlineUserPresenceSteam : public FOnlineUserPresence
{
public:
	uint32 bIsAFriend:1;

	FOnlineUserPresenceSteam()
	{
		Reset();
		bIsAFriend = false;
	}

	void Update(const FUniqueNetIdSteam& FriendId);
};


class FOnlinePresenceSteam : public IOnlinePresence
{
	/** The steam friends interface to use when interacting with Steam */
	class ISteamFriends* SteamFriendsPtr;

	/** Cached pointer to owning subsystem */
	class FOnlineSubsystemSteam* SteamSubsystem;

	/** Hide the default constructor */
	FOnlinePresenceSteam();

	/** All cached presence data we ever get */
	TUniqueNetIdMap<TSharedRef<FOnlineUserPresenceSteam>> CachedPresence;

	/** Delegates for non-friend users that were called with QueryPresence */
	TUniqueNetIdMap<TSharedRef<const FOnPresenceTaskCompleteDelegate>> DelayedPresenceDelegates;

PACKAGE_SCOPE:
	FOnlinePresenceSteam(class FOnlineSubsystemSteam* InSubsystem);

	/** Called from Steam callback events */
	void UpdatePresenceForUser(const FUniqueNetId& User);

public:

	//~ Begin IOnlinePresence Interface
	virtual void SetPresence(const FUniqueNetId& User, const FOnlineUserPresenceStatus& Status, const FOnPresenceTaskCompleteDelegate& Delegate = FOnPresenceTaskCompleteDelegate()) override;
	virtual void QueryPresence(const FUniqueNetId& User, const FOnPresenceTaskCompleteDelegate& Delegate = FOnPresenceTaskCompleteDelegate()) override;
	virtual EOnlineCachedResult::Type GetCachedPresence(const FUniqueNetId& User, TSharedPtr<FOnlineUserPresence>& OutPresence) override;
	virtual EOnlineCachedResult::Type GetCachedPresenceForApp(const FUniqueNetId& LocalUserId, const FUniqueNetId& User, const FString& AppId, TSharedPtr<FOnlineUserPresence>& OutPresence) override;
	//~ End IOnlinePresence Interface

	virtual ~FOnlinePresenceSteam() {}
};

typedef TSharedPtr<FOnlinePresenceSteam, ESPMode::ThreadSafe> FOnlinePresenceSteamPtr;
