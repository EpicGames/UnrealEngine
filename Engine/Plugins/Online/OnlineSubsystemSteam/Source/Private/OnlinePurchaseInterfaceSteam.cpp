// Copyright Epic Games, Inc. All Rights Reserved.
#include "OnlinePurchaseInterfaceSteam.h"
#include "OnlineSubsystemSteam.h"
#include "OnlineError.h"
#include "Misc/ConfigCacheIni.h"
#include "Interfaces/OnlineStoreInterfaceV2.h"

#define DYNAMIC_FIELD_ITEM_CATEGORY TEXT("ItemCategory")
#define DYNAMIC_FIELD_ITEM_NAMESPACE TEXT("ItemNamespace")

FOnlinePurchaseSteam::FOnlinePurchaseSteam(FOnlineSubsystemSteam* InSteamSubsystem)
	: SteamSubsystem(InSteamSubsystem)
{

#if USE_STEAM_DEBUG_PURCHASING_LINK
	TSharedRef<FSteamPurchasingServerLinkDebug> DebugLink = MakeShared<FSteamPurchasingServerLinkDebug>();
	RegisterServerLink(DebugLink);
	DebugLink->OffersUpdatedDelegate.ExecuteIfBound(DebugLink->ConfigCategories, DebugLink->ConfigMtxns);
#endif // USE_STEAM_DEBUG_PURCHASING_LINK

}

FOnlinePurchaseSteam::~FOnlinePurchaseSteam()
{

}

bool FOnlinePurchaseSteam::IsAllowedToPurchase(const FUniqueNetId& UserId)
{
	return true;
}

bool FOnlinePurchaseSteam::IsAllowedToPurchase(const FUniqueNetId& UserId, FOnlineError& Error)
{	
	Error = FOnlineError(EOnlineErrorResult::Success);
	return true;
}

void FOnlinePurchaseSteam::Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseCheckoutComplete& Delegate)
{
	if(!ServerLink.IsValid())
	{
		SteamSubsystem->ExecuteNextTick([Delegate]() {
			UE_LOG_ONLINE_PURCHASE(Error, TEXT("Could not checkout: Steam Server Link is invalid"));

			const TSharedRef<FPurchaseReceipt> PurchaseReceipt = MakeShared<FPurchaseReceipt>();
			PurchaseReceipt->TransactionState = EPurchaseTransactionState::Failed;

			Delegate.ExecuteIfBound(FOnlineError(EOnlineErrorResult::Unknown), PurchaseReceipt);
		});
		return;
	}

	TArray<FSteamPurchaseDef> Mtxns;
	for(const FPurchaseCheckoutRequest::FPurchaseOfferEntry& PurchaseOffer : CheckoutRequest.PurchaseOffers)
	{
		FSteamPurchaseDef& PurchaseDef = Mtxns.Emplace_GetRef();
		PurchaseDef.Quantity = PurchaseOffer.Quantity;
		PurchaseDef.Mtxn->DynamicFields.Add(DYNAMIC_FIELD_ITEM_NAMESPACE, PurchaseOffer.OfferNamespace);
		PurchaseDef.Mtxn->OfferId = PurchaseOffer.OfferId;
	}

	ServerLink->InitiateTransaction(UserId, Mtxns, Delegate);
}

void FOnlinePurchaseSteam::Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseReceiptlessCheckoutComplete& Delegate)
{
	Checkout(UserId, CheckoutRequest, FOnPurchaseCheckoutComplete::CreateLambda([this, &Delegate](const FOnlineError& Result, const TSharedRef<FPurchaseReceipt>& Receipt)
	{
		Delegate.ExecuteIfBound(Result);
	}));
}

void FOnlinePurchaseSteam::FinalizePurchase(const FUniqueNetId& UserId, const FString& ReceiptId)
{
	ServerLink->FinalizePurchase(UserId, ReceiptId);
}

void FOnlinePurchaseSteam::RedeemCode(const FUniqueNetId& UserId, const FRedeemCodeRequest& RedeemCodeRequest, const FOnPurchaseRedeemCodeComplete& Delegate)
{
	SteamSubsystem->ExecuteNextTick([Delegate]() {
		Delegate.ExecuteIfBound(FOnlineError(EOnlineErrorResult::Unknown), MakeShared<FPurchaseReceipt>());
	});
}

void FOnlinePurchaseSteam::QueryReceipts(const FUniqueNetId& UserId, bool bRestoreReceipts, const FOnQueryReceiptsComplete& Delegate)
{
	CachedDLC.Empty();
	int32 DLCCount = SteamApps()->GetDLCCount();
	for (int i = 0; i < DLCCount; ++i) {
		AppId_t OutAppId;
		bool OutbAvailable;
		char OutName[128];
		bool bSuccess = SteamApps()->BGetDLCDataByIndex(i, &OutAppId, &OutbAvailable, OutName, 128);
		if (bSuccess) {
			FPurchaseReceipt& DLCRef = CachedDLC.Emplace_GetRef();
			DLCRef.AddReceiptOffer(FOfferNamespace(), ANSI_TO_TCHAR(OutName), 1);
		}
	}
}

void FOnlinePurchaseSteam::GetReceipts(const FUniqueNetId& UserId, TArray<FPurchaseReceipt>& OutReceipts) const
{
	OutReceipts.Empty();
	for(const FPurchaseReceipt& Receipt : CachedDLC)
	{
		OutReceipts.Add(Receipt);
	}
}

void FOnlinePurchaseSteam::FinalizeReceiptValidationInfo(const FUniqueNetId& UserId, FString& InReceiptValidationInfo, const FOnFinalizeReceiptValidationInfoComplete& Delegate)
{
	ServerLink->FinalizeReceiptValidationInfo(UserId, InReceiptValidationInfo, Delegate);
}

void FOnlinePurchaseSteam::RegisterServerLink(TSharedRef<ISteamPurchasingServerLink> InServerLink)
{
	if(ensure(!ServerLink.IsValid()))
	{
		ServerLink = InServerLink;
		ServerLink->OffersUpdatedDelegate.BindRaw(this, &FOnlinePurchaseSteam::OnServerLinkOffersUpdated);
	}
}

void FOnlinePurchaseSteam::OnServerLinkOffersUpdated(TArray<FString> Categories, TArray<FOnlineStoreOfferRef> Mtxns)
{
	CachedCategories = Categories;
	CachedMtxns = Mtxns;
}


void FOnlinePurchaseSteam::GetStoreCategories(TArray<FOnlineStoreCategory>& OutCategories) const
{
	OutCategories.Empty();
	for(int i = 0; i < CachedCategories.Num(); i++)
	{
		FOnlineStoreCategory& NewCategory = OutCategories.Emplace_GetRef();
		NewCategory.Id = FString::Printf(TEXT("%d"), i);
		NewCategory.Description = FText::FromString(CachedCategories[i]);
	}
}

void FOnlinePurchaseSteam::GetStoreOffers(TArray<FOnlineStoreOfferRef>& OutOffers) const
{
	OutOffers.Empty();
	for(const FOnlineStoreOfferRef& CachedMtxn : CachedMtxns)
	{
		OutOffers.Add(CachedMtxn);
	}
}

TSharedPtr<FOnlineStoreOffer> FOnlinePurchaseSteam::GetStoreOffer(const FUniqueOfferId& OfferId) const
{
	for (const FOnlineStoreOfferRef& CachedMtxn : CachedMtxns)
	{
		if(CachedMtxn->OfferId.Equals(OfferId))
		{
			return CachedMtxn;
		}
	}
	return nullptr;
}


#if USE_STEAM_DEBUG_PURCHASING_LINK
FSteamPurchasingServerLinkDebug::FSteamPurchasingServerLinkDebug()
{
	TArray<FString> ConfigMtxnStrings;
	if (GConfig->GetArray(TEXT("OnlineSubsystemSteam"), TEXT("StaticMicrotransactions"), ConfigMtxnStrings, GEngineIni))
	{
		for(const FString& Mtxn : ConfigMtxnStrings)
		{
			FString ItemId, ItemNamespace, ItemDescription, ItemCost, ItemTitle, ItemCategory;

			FParse::Value(*Mtxn, TEXT("Id="), ItemId);
			FParse::Value(*Mtxn, TEXT("Namespace="), ItemNamespace);
			FParse::Value(*Mtxn, TEXT("Title="), ItemTitle);
			FParse::Value(*Mtxn, TEXT("Description="), ItemDescription);
			FParse::Value(*Mtxn, TEXT("Cost="), ItemCost);
			FParse::Value(*Mtxn, TEXT("Category="), ItemCategory);

			FOnlineStoreOfferRef& NewMtxn = ConfigMtxns.Add_GetRef(MakeShared<FOnlineStoreOffer>());
			NewMtxn->Description = FText::FromString(ItemDescription);
			NewMtxn->Title = FText::FromString(ItemTitle);
			NewMtxn->OfferId = ItemId;
			NewMtxn->DynamicFields.Add(DYNAMIC_FIELD_ITEM_NAMESPACE, ItemNamespace);
			NewMtxn->DynamicFields.Add(DYNAMIC_FIELD_ITEM_CATEGORY, ItemCategory);
			NewMtxn->RegularPriceText = FText::FromString(ItemCost);
		}
	}
	GConfig->GetArray(TEXT("OnlineSubsystemSteam"), TEXT("StaticMicrotransactionCategories"), ConfigCategories, GEngineIni);
}

void FSteamPurchasingServerLinkDebug::InitiateTransaction(const FUniqueNetId& UserId, TArray<FSteamPurchaseDef> Mtxns, const FOnPurchaseCheckoutComplete& Delegate)
{
	TSharedRef<FPurchaseReceipt> Receipt = MakeShared<FPurchaseReceipt>();

	for(const FSteamPurchaseDef& PurchaseDef : Mtxns)
	{
		FString Namespace;
		if(PurchaseDef.Mtxn->DynamicFields.Contains(DYNAMIC_FIELD_ITEM_NAMESPACE))
		{
			Namespace = PurchaseDef.Mtxn->DynamicFields.FindChecked(DYNAMIC_FIELD_ITEM_NAMESPACE);
		}
		Receipt->AddReceiptOffer(Namespace, PurchaseDef.Mtxn->OfferId, PurchaseDef.Quantity);
	}

	Delegate.ExecuteIfBound(FOnlineError::Success(), Receipt);
}

#endif //USE_STEAM_DEBUG_PURCHASING_LINK