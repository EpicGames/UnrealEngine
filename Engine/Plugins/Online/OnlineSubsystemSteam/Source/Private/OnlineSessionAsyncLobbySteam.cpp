// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSessionAsyncLobbySteam.h"
#include "Online/OnlineSessionNames.h"
#include "SocketSubsystem.h"
#include "OnlineSessionInterfaceSteam.h"
#include "OnlinePingInterfaceSteam.h"
#include "IPAddressSteam.h"
#include "SteamSessionKeys.h"
#include "SteamUtilities.h"
#include "Misc/ConfigCacheIni.h"
#include "OnlineSubsystemSteam.h"

/** 
 * Helper function to convert enums 
 * Converts comparison ops into Steam equivalents
 */
inline ELobbyComparison ToSteamLobbyCompareOp(EOnlineComparisonOp::Type InComparisonOp)
{			
	switch (InComparisonOp)
	{
	case EOnlineComparisonOp::NotEquals:
		return k_ELobbyComparisonNotEqual;
	case EOnlineComparisonOp::GreaterThan:
		return k_ELobbyComparisonGreaterThan;
	case EOnlineComparisonOp::GreaterThanEquals:
		return k_ELobbyComparisonEqualToOrGreaterThan;
	case EOnlineComparisonOp::LessThan:
		return k_ELobbyComparisonLessThan;
	case EOnlineComparisonOp::LessThanEquals:
		return k_ELobbyComparisonEqualToOrLessThan;
	case EOnlineComparisonOp::Near:
	case EOnlineComparisonOp::Equals:
	default:
		return k_ELobbyComparisonEqual;
	}
}

/** 
 * Helper function to convert enums 
 * Converts Steam comparison ops into engine equivalents
 */
inline EOnlineComparisonOp::Type FromSteamLobbyCompareOp(ELobbyComparison InComparisonOp)
{
	switch (InComparisonOp)
	{
	case k_ELobbyComparisonNotEqual:
		return EOnlineComparisonOp::NotEquals;
	case k_ELobbyComparisonGreaterThan:
		return EOnlineComparisonOp::GreaterThan;
	case k_ELobbyComparisonEqualToOrGreaterThan:
		return EOnlineComparisonOp::GreaterThanEquals;
	case k_ELobbyComparisonLessThan: 
		return EOnlineComparisonOp::LessThan;
	case k_ELobbyComparisonEqualToOrLessThan:
		return EOnlineComparisonOp::LessThanEquals;
	case k_ELobbyComparisonEqual:
	default:
		return EOnlineComparisonOp::Equals;
	}
}

/**
 *	Generate the proper lobby type from session settings
 *
 * @param SessionSettings current settings for the session
 *
 * @return type of lobby to generate, defaulting to private if not advertising and public otherwise
 */
ELobbyType BuildLobbyType(FOnlineSessionSettings* SessionSettings)
{
	if (!SessionSettings->bIsLANMatch)
	{
		if (SessionSettings->bShouldAdvertise)
		{
			if (SessionSettings->bAllowJoinViaPresenceFriendsOnly)
			{
				// Presence implies invites allowed
				return k_ELobbyTypeFriendsOnly;
			}
			else if (SessionSettings->bAllowInvites && !SessionSettings->bAllowJoinViaPresence)
			{
				// Invite Only
				return k_ELobbyTypePrivate;
			}
			else //bAllowJoinViaPresence
			{
				// Otherwise public
				return k_ELobbyTypePublic;
			}
		}
	}

	// Not really advertising as this isn't findable
	return k_ELobbyTypePrivate;
}

/**
 *	Get all relevant FOnlineSessionSettings data as a series of Key,Value pairs
 *
 * @param SessionSettings session settings to get key, value pairs from
 * @param KeyValuePairs key value pair structure to add to
 */
static void GetLobbyKeyValuePairsFromSessionSettings(const FOnlineSessionSettings& SessionSettings, FSteamSessionKeyValuePairs& KeyValuePairs)
{
	int32 BitShift = 0;
	int32 SessionFlags = 0;
	SessionFlags |= (SessionSettings.bShouldAdvertise ? 1 : 0) << BitShift++;
	SessionFlags |= (SessionSettings.bAllowJoinInProgress ? 1 : 0) << BitShift++;
	SessionFlags |= (SessionSettings.bIsLANMatch ? 1 : 0) << BitShift++;
	SessionFlags |= (SessionSettings.bIsDedicated ? 1 : 0) << BitShift++;
	SessionFlags |= (SessionSettings.bUsesStats ? 1 : 0) << BitShift++;
	SessionFlags |= (SessionSettings.bAllowInvites ? 1 : 0) << BitShift++;
	SessionFlags |= (SessionSettings.bUsesPresence ? 1 : 0) << BitShift++;
	SessionFlags |= (SessionSettings.bUseLobbiesIfAvailable ? 1 : 0) << BitShift++;
	SessionFlags |= (SessionSettings.bAllowJoinViaPresence ? 1 : 0) << BitShift++;
	SessionFlags |= (SessionSettings.bAllowJoinViaPresenceFriendsOnly ? 1 : 0) << BitShift++;
	SessionFlags |= (SessionSettings.bAntiCheatProtected ? 1 : 0) << BitShift++;

	KeyValuePairs.Add(STEAMKEY_NUMPUBLICCONNECTIONS, FString::FromInt(SessionSettings.NumPublicConnections));
	KeyValuePairs.Add(STEAMKEY_NUMPRIVATECONNECTIONS, FString::FromInt(SessionSettings.NumPrivateConnections));
	KeyValuePairs.Add(STEAMKEY_SESSIONFLAGS, FString::FromInt(SessionFlags));
	KeyValuePairs.Add(STEAMKEY_BUILDUNIQUEID, FString::FromInt(SessionSettings.BuildUniqueId));

	FString KeyStr;
	for (FSessionSettings::TConstIterator It(SessionSettings.Settings); It; ++It)
	{
		FName Key = It.Key();
		const FOnlineSessionSetting& Setting = It.Value();
		if (Setting.AdvertisementType >= EOnlineDataAdvertisementType::ViaOnlineService)
		{
			if (SessionKeyToSteamKey(Key, Setting.Data, KeyStr))
			{
				FString SettingStr = Setting.Data.ToString();
				if (!SettingStr.IsEmpty())
				{
					KeyValuePairs.Add(KeyStr, SettingStr);
				}
				else
				{
					UE_LOG_ONLINE_SESSION(Warning, TEXT("Empty session setting %s %s of type %s"), *Key.ToString(), *Setting.ToString(), EOnlineKeyValuePairDataType::ToString(Setting.Data.GetType()));
				}
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Unsupported session setting %s %s of type %s"), *Key.ToString(), *Setting.ToString(), EOnlineKeyValuePairDataType::ToString(Setting.Data.GetType()));
			}
		}
	}
}

/**
 *	Get all relevant FSessionInfoSteam data as a series of Key,Value pairs
 *
 * @param SessionInfo session info to get key, value pairs from
 * @param KeyValuePairs key value pair structure to add to
 */
static void GetLobbyKeyValuePairsFromSessionInfo(const FOnlineSessionInfoSteam& SessionInfo, FSteamSessionKeyValuePairs& KeyValuePairs)
{
	// This should pretty much always be false.
	if (SessionInfo.HostAddr.IsValid())
	{
		uint32 HostAddr;
		SessionInfo.HostAddr->GetIp(HostAddr);
		KeyValuePairs.Add(STEAMKEY_HOSTIP, FString::FromInt(HostAddr));
		KeyValuePairs.Add(STEAMKEY_HOSTPORT, FString::FromInt(SessionInfo.HostAddr->GetPort()));
	}

	// Lobbies are exclusively P2P.
	KeyValuePairs.Add(STEAMKEY_CONNECTIONMETHOD, FString::Printf(TEXT("%s"), *LexToString(FSteamConnectionMethod::P2P)));

	if (SessionInfo.SteamP2PAddr.IsValid())
	{
		KeyValuePairs.Add(STEAMKEY_P2PADDR, SessionInfo.SteamP2PAddr->ToString(false));
		KeyValuePairs.Add(STEAMKEY_P2PPORT, FString::FromInt(SessionInfo.SteamP2PAddr->GetPort()));
	}
}

/**
 *	Get all relevant FSession data as a series of Key,Value pairs
 *
 * @param Session session data to get key, value pairs from
 * @param KeyValuePairs key value pair structure to add to
 */
static void GetLobbyKeyValuePairsFromSession(const FOnlineSession* Session, FSteamSessionKeyValuePairs& KeyValuePairs)
{
	FUniqueNetIdSteamPtr SteamId = StaticCastSharedPtr<const FUniqueNetIdSteam>(Session->OwningUserId);
	if (SteamId.IsValid())
	{
		KeyValuePairs.Add(STEAMKEY_OWNINGUSERID, SteamId->ToString());
	}
	KeyValuePairs.Add(STEAMKEY_OWNINGUSERNAME, Session->OwningUserName);
	KeyValuePairs.Add(STEAMKEY_NUMOPENPRIVATECONNECTIONS, FString::FromInt(Session->NumOpenPrivateConnections));
	KeyValuePairs.Add(STEAMKEY_NUMOPENPUBLICCONNECTIONS, FString::FromInt(Session->NumOpenPublicConnections));

	if (Session->SessionInfo.IsValid())
	{
		TSharedPtr<FOnlineSessionInfoSteam> SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoSteam>(Session->SessionInfo);
		GetLobbyKeyValuePairsFromSessionInfo(*SessionInfo, KeyValuePairs);
	}

	// Write in the ping information if the host is using SteamSockets
	FOnlineSubsystemSteam* SteamSubsystem = static_cast<FOnlineSubsystemSteam*>(IOnlineSubsystem::Get(STEAM_SUBSYSTEM));
	if (SteamSubsystem && SteamSubsystem->GetPingInterface().IsValid() && SteamSubsystem->GetPingInterface()->IsUsingP2PRelays())
	{
		FString PingData = SteamSubsystem->GetPingInterface()->GetHostPingData();
		// Don't bother writing empty data.
		if (!PingData.IsEmpty())
		{
			KeyValuePairs.Add(STEAMKEY_P2PPING, PingData);
		}
	}

	GetLobbyKeyValuePairsFromSessionSettings(Session->SessionSettings, KeyValuePairs);
}

/**
 *	Populate an FSession data structure from the data stored with a lobby
 * Expects a certain number of keys to be present otherwise this will fail
 *
 * @param SteamSubsystem the online subsystem to use
 * @param LobbyId the Steam lobby to fill data from
 * @param Session empty session structure to fill in
 * @param SearchData if this is provided and the Steam session supports SteamSockets, the ping for the lobby will be written into the search result data.
 *
 * @return true if successful, false otherwise
 */
bool FillSessionFromLobbyData(FOnlineSubsystemSteam* SteamSubsystem, const FUniqueNetIdSteam& LobbyId, FOnlineSession& Session, FOnlineSessionSearchResult* SearchData)
{
	bool bSuccess = true;

	ISteamMatchmaking* SteamMatchmakingPtr = SteamMatchmaking();
	check(SteamMatchmakingPtr);

	// Empty session settings
	Session.SessionSettings.Settings.Empty();

	// Create the session info
	TSharedPtr<FOnlineSessionInfoSteam> SessionInfo = MakeShared<FOnlineSessionInfoSteam>(ESteamSession::LobbySession, LobbyId);
	TSharedRef<FInternetAddr> HostAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();
	TSharedRef<FInternetAddrSteam> SteamP2PAddr = MakeShareable(new FInternetAddrSteam);
	
	// Make sure we hit the important keys
	int32 KeysFound = 0;
	int32 HostKeysFound = 0;
	int32 SteamAddrKeysFound = 0;

	const int32 LobbyDataBufferSize = 2000; // The Limit on Steam is 8192, but we won't use that much.
	ANSICHAR Key[LobbyDataBufferSize];
	ANSICHAR Value[LobbyDataBufferSize];

	// Lobby data update
	int32 LobbyDataCount = SteamMatchmakingPtr->GetLobbyDataCount(LobbyId);
	for (int32 LobbyDataIdx=0; LobbyDataIdx<LobbyDataCount; LobbyDataIdx++)
	{
		if (!SteamMatchmakingPtr->GetLobbyDataByIndex(LobbyId, LobbyDataIdx, Key, LobbyDataBufferSize, Value, LobbyDataBufferSize))
		{
			// Treat any failure to get lobby data as failed search result
			KeysFound = 0;
			break;
		}

		if (FCStringAnsi::Stricmp(Key, STEAMKEY_NUMPUBLICCONNECTIONS) == 0)
		{
			Session.SessionSettings.NumPublicConnections = FCString::Atoi(ANSI_TO_TCHAR(Value));
			KeysFound++;
		}
		else if (FCStringAnsi::Stricmp(Key, STEAMKEY_NUMPRIVATECONNECTIONS) == 0)
		{
			Session.SessionSettings.NumPrivateConnections = FCString::Atoi(ANSI_TO_TCHAR(Value));
			KeysFound++;
		}
		else if (FCStringAnsi::Stricmp(Key, STEAMKEY_SESSIONFLAGS) == 0)
		{
			int32 BitShift = 0;
			int32 SessionFlags = FCString::Atoi(ANSI_TO_TCHAR(Value));
			Session.SessionSettings.bShouldAdvertise = (SessionFlags & (1 << BitShift++)) ? true : false;
			Session.SessionSettings.bAllowJoinInProgress = (SessionFlags & (1 << BitShift++)) ? true : false;
			Session.SessionSettings.bIsLANMatch = (SessionFlags & (1 << BitShift++)) ? true : false;
			Session.SessionSettings.bIsDedicated = (SessionFlags & (1 << BitShift++)) ? true : false;
			Session.SessionSettings.bUsesStats = (SessionFlags & (1 << BitShift++)) ? true : false;
			Session.SessionSettings.bAllowInvites = (SessionFlags & (1 << BitShift++)) ? true : false;
			Session.SessionSettings.bUsesPresence = (SessionFlags & (1 << BitShift++)) ? true : false;
			Session.SessionSettings.bUseLobbiesIfAvailable = (SessionFlags & (1 << BitShift++)) ? true : false;
			Session.SessionSettings.bAllowJoinViaPresence = (SessionFlags & (1 << BitShift++)) ? true : false;
			Session.SessionSettings.bAllowJoinViaPresenceFriendsOnly = (SessionFlags & (1 << BitShift++)) ? true : false;
			Session.SessionSettings.bAntiCheatProtected = (SessionFlags & (1 << BitShift++)) ? true : false;
			KeysFound++;
		}
		else if (FCStringAnsi::Stricmp(Key, STEAMKEY_BUILDUNIQUEID) == 0)
		{
			Session.SessionSettings.BuildUniqueId = FCString::Atoi(ANSI_TO_TCHAR(Value));
			KeysFound++;
		}
		else if (FCStringAnsi::Stricmp(Key, STEAMKEY_OWNINGUSERID) == 0)
		{
			uint64 UniqueId = FCString::Atoi64(ANSI_TO_TCHAR(Value));
			if (UniqueId != 0)
			{
				Session.OwningUserId = FUniqueNetIdSteam::Create(UniqueId);
				KeysFound++;
			}
		}
		else if (FCStringAnsi::Stricmp(Key, STEAMKEY_OWNINGUSERNAME) == 0)
		{
			if (FCString::Strlen(UTF8_TO_TCHAR(Value)) > 0)
			{
				Session.OwningUserName = UTF8_TO_TCHAR(Value);
				KeysFound++;
			}
		}
		else if (FCStringAnsi::Stricmp(Key, STEAMKEY_NUMOPENPRIVATECONNECTIONS) == 0)
		{
			Session.NumOpenPrivateConnections = FCString::Atoi(ANSI_TO_TCHAR(Value));
			KeysFound++;
		}
		else if (FCStringAnsi::Stricmp(Key, STEAMKEY_NUMOPENPUBLICCONNECTIONS) == 0)
		{
			Session.NumOpenPublicConnections = FCString::Atoi(ANSI_TO_TCHAR(Value));
			KeysFound++;
		}
		else if (FCStringAnsi::Strcmp(Key, STEAMKEY_P2PPING) == 0)
		{
			if (SteamSubsystem && SearchData != nullptr && SteamSubsystem->GetPingInterface().IsValid())
			{
				int32 PingResult = SteamSubsystem->GetPingInterface()->GetPingFromHostData(ANSI_TO_TCHAR(Value));
				// If the ping was successful, we'll want to write the data. Otherwise let it stay at the default value (of 999)
				if (PingResult >= 0)
				{
					SearchData->PingInMs = PingResult;
				}
			}
		}
		else if (FCStringAnsi::Stricmp(Key, STEAMKEY_HOSTIP) == 0)
		{
			uint32 HostIp = FCString::Atoi(ANSI_TO_TCHAR(Value));
			if (HostIp != 0)
			{
				HostAddr->SetIp(HostIp);
				HostKeysFound++;
			}
		}
		else if (FCStringAnsi::Stricmp(Key, STEAMKEY_HOSTPORT) == 0)
		{
			int32 Port = FCString::Atoi(ANSI_TO_TCHAR(Value));
			if (Port != 0)
			{
				HostAddr->SetPort(Port);
				HostKeysFound++;
			}
		}
		else if (FCStringAnsi::Stricmp(Key, STEAMKEY_CONNECTIONMETHOD) == 0)
		{
			SessionInfo->ConnectionMethod = ToConnectionMethod(ANSI_TO_TCHAR(Value));
			++KeysFound;
		}
		else if (FCStringAnsi::Stricmp(Key, STEAMKEY_P2PADDR) == 0)
		{
			FString KeyValue = ANSI_TO_TCHAR(Value);
			// Remove any protocol flags from the start.
			KeyValue.RemoveFromStart(STEAM_URL_PREFIX);

			uint64 SteamAddr = FCString::Atoi64(*KeyValue);
			if (SteamAddr != 0)
			{
				SteamP2PAddr->SetSteamID(CSteamID(SteamAddr));
				SteamAddrKeysFound++;
			}
		}
		else if (FCStringAnsi::Stricmp(Key, STEAMKEY_P2PPORT) == 0)
		{
			int32 Port = FCString::Atoi(ANSI_TO_TCHAR(Value));
			SteamP2PAddr->SetPort(Port);
			SteamAddrKeysFound++;
		}
		else
		{
			FName NewKey;
			FOnlineSessionSetting NewSetting;
			if (SteamKeyToSessionSetting(ANSI_TO_TCHAR(Key), Value, NewKey, NewSetting))
			{
				Session.SessionSettings.Set(NewKey, NewSetting);
			}
			else
			{
				bSuccess = false;
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to parse setting from key %s value %s"), ANSI_TO_TCHAR(Key), ANSI_TO_TCHAR(Value));
			}
		}
	}

	// Verify success with all required keys found
	if (bSuccess && (KeysFound == STEAMKEY_NUMREQUIREDLOBBYKEYS) && (HostKeysFound == 2 || SteamAddrKeysFound == 2))
	{
		int32 BuildUniqueId = GetBuildUniqueId();
		if (Session.SessionSettings.BuildUniqueId == BuildUniqueId)
		{
			if (HostKeysFound == 2)
			{
				SessionInfo->HostAddr = HostAddr;
			}

			if (SteamAddrKeysFound == 2)
			{
				SessionInfo->SteamP2PAddr = SteamP2PAddr;
			}

			Session.SessionInfo = SessionInfo;
			return true;
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Removed incompatible build: ServerBuildUniqueId = 0x%08x, GetBuildUniqueId() = 0x%08x"),
				Session.SessionSettings.BuildUniqueId, BuildUniqueId);
		}
	}

	return false;
}

/**
 *	Populate an FSession data structure from the data stored with the members of the lobby
 *
 * @param LobbyId the Steam lobby to fill data from
 * @param Session session structure to fill in
 *
 * @return true if successful, false otherwise
 */
bool FillMembersFromLobbyData(const FUniqueNetIdSteam& LobbyId, FNamedOnlineSession& Session)
{
	bool bSuccess = true;

	ISteamMatchmaking* SteamMatchmakingPtr = SteamMatchmaking();
	check(SteamMatchmakingPtr);

	// Attempt to parse the lobby members
	int32 LobbyMemberCount = SteamMatchmakingPtr->GetNumLobbyMembers(LobbyId);
	int32 MaxLobbyMembers = SteamMatchmakingPtr->GetLobbyMemberLimit(LobbyId);
	if (MaxLobbyMembers > 0)
	{
		// Keep the number of connections current
		Session.NumOpenPublicConnections = MaxLobbyMembers - LobbyMemberCount;

		if (SteamMatchmakingPtr->GetLobbyOwner(LobbyId) == SteamUser()->GetSteamID())
		{
			// Auto update joinability based on lobby population
			bool bLobbyJoinable = Session.SessionSettings.bAllowJoinInProgress && (LobbyMemberCount < MaxLobbyMembers);

			UE_LOG_ONLINE_SESSION(Log, TEXT("Updating lobby joinability to %s."), bLobbyJoinable ? TEXT("true") : TEXT("false"));
			if (!SteamMatchmakingPtr->SetLobbyJoinable(LobbyId, bLobbyJoinable))
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to update lobby joinability."));
				bSuccess = false;
			}
		}
	}

	return bSuccess;
}

/**
 *	Get a human readable description of task
 */
FString FOnlineAsyncTaskSteamCreateLobby::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncTaskSteamCreateLobby bWasSuccessful: %d LobbyId: %llu LobbyType: %d Result: %s"), WasSuccessful(), CallbackResults.m_ulSteamIDLobby, (int32)LobbyType, *SteamResultString(CallbackResults.m_eResult));
}

/**
 * Give the async task time to do its work
 * Can only be called on the async task manager thread
 */
void FOnlineAsyncTaskSteamCreateLobby::Tick()
{
	ISteamUtils* SteamUtilsPtr = SteamUtils();
	check(SteamUtilsPtr);

	if (!bInit)
	{
		CallbackHandle = SteamMatchmaking()->CreateLobby(LobbyType, MaxLobbyMembers);
		bInit = true;
	}

	if (CallbackHandle != k_uAPICallInvalid)
	{
		bool bFailedCall = false; 

		// Poll for completion status
		bIsComplete = SteamUtilsPtr->IsAPICallCompleted(CallbackHandle, &bFailedCall) ? true : false; 
		if (bIsComplete) 
		{ 
			bool bFailedResult;
			// Retrieve the callback data from the request
			bool bSuccessCallResult = SteamUtilsPtr->GetAPICallResult(CallbackHandle, &CallbackResults, sizeof(CallbackResults), CallbackResults.k_iCallback, &bFailedResult); 
			bWasSuccessful = (bSuccessCallResult ? true : false) &&
				(!bFailedCall ? true : false) &&
				(!bFailedResult ? true : false) &&
				((CallbackResults.m_eResult == k_EResultOK) ? true : false) &&
				((CallbackResults.m_ulSteamIDLobby > 0 ? true : false));
		} 
	}
	else
	{
		// Invalid API call
		bIsComplete = true;
		bWasSuccessful = false;
	}
}

/**
 * Give the async task a chance to marshal its data back to the game thread
 * Can only be called on the game thread by the async task manager
 */
void FOnlineAsyncTaskSteamCreateLobby::Finalize()
{
	FOnlineSessionSteamPtr SessionInt = StaticCastSharedPtr<FOnlineSessionSteam>(Subsystem->GetSessionInterface());

	if (bWasSuccessful)
	{
		FNamedOnlineSession* Session = SessionInt->GetNamedSession(SessionName);
		if (Session)
		{
			Session->LocalOwnerId = FUniqueNetIdSteam::Create(SteamUser()->GetSteamID());

			ISteamMatchmaking* SteamMatchMakingPtr = SteamMatchmaking();
			check(SteamMatchMakingPtr);
			const FUniqueNetIdSteamRef LobbyId = FUniqueNetIdSteam::Create(CallbackResults.m_ulSteamIDLobby);

			// Setup the host session info now that we have a lobby id
			TSharedRef<FOnlineSessionInfoSteam> NewSessionInfo = MakeShared<FOnlineSessionInfoSteam>(ESteamSession::LobbySession, *LobbyId);
			// Lobby sessions don't have a valid IP
			NewSessionInfo->HostAddr = NULL;
			// Copy the P2P addr
			TSharedRef<FInternetAddrSteam> SteamAddr = MakeShared<FInternetAddrSteam>(SteamUser()->GetSteamID());
			SteamAddr->SetPort(Subsystem->GetGameServerGamePort());
			NewSessionInfo->SteamP2PAddr = SteamAddr;

			// Set the info on the session
			Session->SessionInfo = NewSessionInfo;

			// Set the game state as pending (not started)
			Session->SessionState = EOnlineSessionState::Pending;

			FSteamSessionKeyValuePairs KeyValuePairs;
			GetLobbyKeyValuePairsFromSession(Session, KeyValuePairs);

			// Register session properties with Steam lobby
			for (FSteamSessionKeyValuePairs::TConstIterator It(KeyValuePairs); It; ++It)
			{
				UE_LOG_ONLINE_SESSION(Verbose, TEXT("Lobby Data (%s, %s)"), *It.Key(), *It.Value());
				if (!SteamMatchMakingPtr->SetLobbyData(*LobbyId, TCHAR_TO_UTF8(*It.Key()), TCHAR_TO_UTF8(*It.Value())))
				{
					bWasSuccessful = false;
					break;
				}
			}

			if (!bWasSuccessful)
			{
				bWasSuccessful = false;
				SteamMatchMakingPtr->LeaveLobby(*LobbyId);
				SessionInt->RemoveNamedSession(SessionName);
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to set lobby data for session %s, cleaning up."), *SessionName.ToString());
			}
			else
			{
				SessionInt->JoinedLobby(*LobbyId);
				SessionInt->RegisterLocalPlayers(Session);
				DumpNamedSession(Session);
			}
		}
	}
	else
	{
		SessionInt->RemoveNamedSession(SessionName);
	}
}

/**
 *	Async task is given a chance to trigger it's delegates
 */
void FOnlineAsyncTaskSteamCreateLobby::TriggerDelegates()
{
	IOnlineSessionPtr SessionInt = Subsystem->GetSessionInterface();
	if (SessionInt.IsValid())
	{
		SessionInt->TriggerOnCreateSessionCompleteDelegates(SessionName, bWasSuccessful);
	}
}

/**
 *	Get a human readable description of task
 */
FString FOnlineAsyncTaskSteamJoinLobby::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncTaskSteamJoinLobby bWasSuccessful: %d Session: %s LobbyId: %s Result: %s"), 
		WasSuccessful(),
		*SessionName.ToString(), 
		*LobbyId->ToDebugString(),
		*SteamChatRoomEnterResponseString((EChatRoomEnterResponse)CallbackResults.m_EChatRoomEnterResponse));
}

/**
 *	Get a human readable description of task
 */
FString FOnlineAsyncTaskSteamUpdateLobby::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncTaskSteamUpdateLobby bWasSuccessful: %d Session: %s"),
		WasSuccessful(),
		*SessionName.ToString());
}

/**
 * Give the async task time to do its work
 * Can only be called on the async task manager thread
 */
void FOnlineAsyncTaskSteamUpdateLobby::Tick() 
{
	bWasSuccessful = false;
	
	IOnlineSessionPtr SessionInt = Subsystem->GetSessionInterface();
	if (SessionInt.IsValid())
	{
		// Grab the session information by name
		FNamedOnlineSession* Session = SessionInt->GetNamedSession(SessionName);
		if (Session)
		{
			TSharedPtr<FOnlineSessionInfoSteam> SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoSteam>(Session->SessionInfo);
			if (SessionInfo.IsValid())
			{
				bool bUsesPresence = Session->SessionSettings.bUsesPresence;
				if (bUsesPresence != NewSessionSettings.bUsesPresence)
				{
					UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't change presence settings on existing session %s, ignoring."), *SessionName.ToString());
				}

				FSteamSessionKeyValuePairs OldKeyValuePairs;
				GetLobbyKeyValuePairsFromSession(Session, OldKeyValuePairs);

				Session->SessionSettings = NewSessionSettings;
				Session->SessionSettings.bUsesPresence = bUsesPresence;

				if (bUpdateOnlineData)
				{
					ISteamMatchmaking* SteamMatchmakingPtr = SteamMatchmaking();
					check(SteamMatchmakingPtr);

					ELobbyType LobbyType = BuildLobbyType(&Session->SessionSettings);
					if (SteamMatchmakingPtr->SetLobbyType(*SessionInfo->SessionId, LobbyType))
					{
						int32 LobbyMemberCount = SteamMatchmakingPtr->GetNumLobbyMembers(*SessionInfo->SessionId);
						int32 NumConnections = Session->SessionSettings.NumPrivateConnections + Session->SessionSettings.NumPublicConnections;

						if (SteamMatchmakingPtr->SetLobbyMemberLimit(*SessionInfo->SessionId, NumConnections))
						{
							int32 MaxLobbyMembers = SteamMatchmakingPtr->GetLobbyMemberLimit(*SessionInfo->SessionId);
							bool bLobbyJoinable = Session->SessionSettings.bAllowJoinInProgress && (LobbyMemberCount < MaxLobbyMembers) && (MaxLobbyMembers != 0);
							if (SteamMatchmakingPtr->SetLobbyJoinable(*SessionInfo->SessionId, bLobbyJoinable))
							{
								bWasSuccessful = true;

								FSteamSessionKeyValuePairs KeyValuePairs;
								GetLobbyKeyValuePairsFromSession(Session, KeyValuePairs);

								// @TODO ONLINE Make sure to only remove/set data that has changed
								// Unregister old session properties with Steam lobby
								for (FSteamSessionKeyValuePairs::TConstIterator It(OldKeyValuePairs); It; ++It)
								{
									UE_LOG_ONLINE_SESSION(Verbose, TEXT("Removing Lobby Data (%s, %s)"), *It.Key(), *It.Value());
									if (!SteamMatchmakingPtr->SetLobbyData(*SessionInfo->SessionId, TCHAR_TO_UTF8(*It.Key()), ""))
									{
										bWasSuccessful = false;
										break;
									}
								}

								if (bWasSuccessful)
								{
									// Register session properties with Steam lobby
									for (FSteamSessionKeyValuePairs::TConstIterator It(KeyValuePairs); It; ++It)
									{
										UE_LOG_ONLINE_SESSION(Verbose, TEXT("Updating Lobby Data (%s, %s)"), *It.Key(), *It.Value());
										if (!SteamMatchmakingPtr->SetLobbyData(*SessionInfo->SessionId, TCHAR_TO_UTF8(*It.Key()), TCHAR_TO_UTF8(*It.Value())))
										{
											bWasSuccessful = false;
											break;
										}
									}
								}
							}
						}
					}
				}
				else
				{
					bWasSuccessful = true;
				}
			}
		}
	}

	bIsComplete = true;
}

/**
 *	Async task is given a chance to trigger it's delegates
 */
void FOnlineAsyncTaskSteamUpdateLobby::TriggerDelegates()
{
	IOnlineSessionPtr SessionInt = Subsystem->GetSessionInterface();
	if (SessionInt.IsValid())
	{
		SessionInt->TriggerOnUpdateSessionCompleteDelegates(SessionName, bWasSuccessful);
	}
}

/**
 * Give the async task time to do its work
 * Can only be called on the async task manager thread
 */
void FOnlineAsyncTaskSteamJoinLobby::Tick()
{
	ISteamUtils* SteamUtilsPtr = SteamUtils();
	check(SteamUtilsPtr);

	if (!bInit)
	{
		CallbackHandle = SteamMatchmaking()->JoinLobby(*LobbyId);
		bInit = true;
	}

	if (CallbackHandle != k_uAPICallInvalid)
	{
		bool bFailedCall = false; 

		// Poll for completion status
		bIsComplete = SteamUtilsPtr->IsAPICallCompleted(CallbackHandle, &bFailedCall) ? true : false; 
		if (bIsComplete) 
		{ 
			bool bFailedResult;
			// Retrieve the callback data from the request
			bool bSuccessCallResult = SteamUtilsPtr->GetAPICallResult(CallbackHandle, &CallbackResults, sizeof(CallbackResults), CallbackResults.k_iCallback, &bFailedResult); 
			bWasSuccessful = (bSuccessCallResult ? true : false) &&
				(!bFailedCall ? true : false) &&
				(!bFailedResult ? true : false) &&
				((CallbackResults.m_EChatRoomEnterResponse == k_EChatRoomEnterResponseSuccess) ? true : false) &&
				(CSteamID(CallbackResults.m_ulSteamIDLobby) == *LobbyId ? true : false);
		} 
	}
	else
	{
		// Invalid API call
		bIsComplete = true;
		bWasSuccessful = false;
	}
}

/**
 * Give the async task a chance to marshal its data back to the game thread
 * Can only be called on the game thread by the async task manager
 */
void FOnlineAsyncTaskSteamJoinLobby::Finalize()
{
	FOnlineSessionSteamPtr SessionInt = StaticCastSharedPtr<FOnlineSessionSteam>(Subsystem->GetSessionInterface());
	if (SessionInt.IsValid())
	{
		if (bWasSuccessful)
		{
			FNamedOnlineSession* Session = SessionInt->GetNamedSession(SessionName);
			if (Session)
			{
				// Session settings were set in the LobbyUpdate async event triggered upon join
				Session->SessionState = EOnlineSessionState::Pending;
				SessionInt->JoinedLobby(*LobbyId);
				SessionInt->RegisterLocalPlayers(Session);
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Session %s not found when trying to join"), *SessionName.ToString());
			}
		}
	}

	if (!bWasSuccessful)
	{
		// Clean up partial create/join
		SessionInt->RemoveNamedSession(SessionName);
	}
}

/**
 *	Async task is given a chance to trigger it's delegates
 */
void FOnlineAsyncTaskSteamJoinLobby::TriggerDelegates()
{
	IOnlineSessionPtr SessionInt = Subsystem->GetSessionInterface();
	if (SessionInt.IsValid())
	{
		SessionInt->TriggerOnJoinSessionCompleteDelegates(SessionName, bWasSuccessful ? EOnJoinSessionCompleteResult::Success : EOnJoinSessionCompleteResult::UnknownError);
	}
}

/**
 *	Get a human readable description of task
 */
FString FOnlineAsyncTaskSteamLeaveLobby::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncTaskSteamLeaveLobby bWasSuccessful: %d SessionName: %s LobbyId: %s"), WasSuccessful(), *SessionName.ToString(), *LobbyId->ToDebugString());
}

/**
 * Give the async task time to do its work
 * Can only be called on the async task manager thread
 */
void FOnlineAsyncTaskSteamLeaveLobby::Tick()
{
	SteamMatchmaking()->LeaveLobby(*LobbyId);
    FOnlineSessionSteamPtr SessionInt = StaticCastSharedPtr<FOnlineSessionSteam>(Subsystem->GetSessionInterface());
	if (SessionInt.IsValid())
	{
		SessionInt->LeftLobby(*LobbyId);
	}

	bIsComplete = true;
	bWasSuccessful = true;
}

/**
 *	Create and trigger the lobby query from the defined search settings 
 */
void FOnlineAsyncTaskSteamFindLobbiesBase::CreateQuery()
{
	check(SteamMatchmakingPtr);

	// Maximum results to return
	if (SearchSettings->MaxSearchResults > 0)
	{
		SteamMatchmakingPtr->AddRequestLobbyListResultCountFilter(SearchSettings->MaxSearchResults);
	}

	// @TODO Online - integrate this filter
	// SteamMatchmakingPtr->AddRequestLobbyListFilterSlotsAvailable( int nSlotsAvailable );

	// Distance of search result from searching client
	SteamMatchmakingPtr->AddRequestLobbyListDistanceFilter(k_ELobbyDistanceFilterDefault);

	for (FSearchParams::TConstIterator It(SearchSettings->QuerySettings.SearchParams); It; ++It)
	{
		const FName Key = It.Key();
		const FOnlineSessionSearchParam& SearchParam = It.Value();

		// Game server keys are skipped
		if (Key == SEARCH_DEDICATED_ONLY || Key == SETTING_MAPNAME || Key == SEARCH_EMPTY_SERVERS_ONLY || Key == SEARCH_SECURE_SERVERS_ONLY || Key == SEARCH_LOBBIES)
		{
			continue;
		}

		FString KeyStr;
		if (SessionKeyToSteamKey(Key, SearchParam.Data, KeyStr))
		{
			if (SearchParam.ComparisonOp == EOnlineComparisonOp::Near)
			{
				//Near filters don't actually filter out values, they just influence how the results are sorted. You can specify multiple near filters, with the first near filter influencing the most, and the last near filter influencing the least.
				switch(SearchParam.Data.GetType())
				{
				case EOnlineKeyValuePairDataType::Int32:
					{
						int32 Value;
						SearchParam.Data.GetValue(Value);
						SteamMatchmakingPtr->AddRequestLobbyListNearValueFilter(TCHAR_TO_UTF8(*KeyStr), Value);
						break;
					}
				default:
					UE_LOG_ONLINE_SESSION(Warning, TEXT("Unable to set search parameter %s"), *SearchParam.ToString());
					break;
				}	
			}
			else
			{
				switch(SearchParam.Data.GetType())
				{
				case EOnlineKeyValuePairDataType::Int32:
					{
						int32 Value;
						SearchParam.Data.GetValue(Value);
						SteamMatchmakingPtr->AddRequestLobbyListNumericalFilter(TCHAR_TO_UTF8(*KeyStr), Value, ToSteamLobbyCompareOp(SearchParam.ComparisonOp));
						break;
					}
				case EOnlineKeyValuePairDataType::Float:
					{
						// @TODO ONLINE - Equality works, but rest untested
						SteamMatchmakingPtr->AddRequestLobbyListStringFilter(TCHAR_TO_UTF8(*KeyStr), TCHAR_TO_UTF8(*SearchParam.Data.ToString()), ToSteamLobbyCompareOp(SearchParam.ComparisonOp));
						break;
					}
				case EOnlineKeyValuePairDataType::String:
					{
						FString Value;
						SearchParam.Data.GetValue(Value);

						if (!Value.IsEmpty())
						{
							SteamMatchmakingPtr->AddRequestLobbyListStringFilter(TCHAR_TO_UTF8(*KeyStr), TCHAR_TO_UTF8(*Value), ToSteamLobbyCompareOp(SearchParam.ComparisonOp));
						}	
						else
						{
							UE_LOG_ONLINE_SESSION(Warning, TEXT("Empty search parameter %s: %s"), *Key.ToString(), *SearchParam.ToString());
						}

						break;
					}
				default:
					UE_LOG_ONLINE_SESSION(Warning, TEXT("Unable to set search parameter %s: %s"), *Key.ToString(), *SearchParam.ToString());
					break;
				}
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Unsupported search setting %s %s of type %s"), *Key.ToString(), *SearchParam.ToString(), EOnlineComparisonOp::ToString(SearchParam.ComparisonOp));
		}
	}
}

/**
 *	Create a search result from a specified lobby id 
 *
 * @param LobbyId lobby to create the search result for
 */
void FOnlineAsyncTaskSteamFindLobbiesBase::ParseSearchResult(const FUniqueNetIdSteam& LobbyId)
{
	FOnlineSessionSearchResult* NewSearchResult = new (SearchSettings->SearchResults) FOnlineSessionSearchResult();
	if (!FillSessionFromLobbyData(Subsystem, LobbyId, NewSearchResult->Session, NewSearchResult))
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Unable to parse search result for lobby '%s'"), *LobbyId.ToDebugString());
		// Remove the failed element
		SearchSettings->SearchResults.RemoveAtSwap(SearchSettings->SearchResults.Num() - 1);
	}
}

/**
 * Give the async task time to do its work
 * Can only be called on the async task manager thread
 */
void FOnlineAsyncTaskSteamFindLobbiesBase::Tick()
{
	ISteamUtils* SteamUtilsPtr = SteamUtils();
	check(SteamUtilsPtr);

	switch (FindLobbiesState)
	{
	case EFindLobbiesState::Init:
		{
			// Don't try to search if the network device is broken
			if (ISocketSubsystem::Get()->HasNetworkDevice())
			{
				// Make sure they are logged in to play online
				if (SteamUser()->BLoggedOn())
				{
					UE_LOG_ONLINE_SESSION(Verbose, TEXT("Starting search for Internet games..."));

					// Setup the filters
					CreateQuery();
					// Start the async search
					CallbackHandle = SteamMatchmakingPtr->RequestLobbyList();
				}
				else
				{
					UE_LOG_ONLINE_SESSION(Warning, TEXT("You must be logged in to an online profile to search for internet games"));
				}
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't search for an internet game without a network connection"));
			}

			if (CallbackHandle == k_uAPICallInvalid)
			{
				bWasSuccessful = false;
				FindLobbiesState = EFindLobbiesState::Finished;
			}
			else
			{
				FindLobbiesState = EFindLobbiesState::RequestLobbyList;
			}
			break;
		}
	case EFindLobbiesState::RequestLobbyList:
		{
			// Poll for completion status
			bool bFailedCall = false;
			if (SteamUtilsPtr->IsAPICallCompleted(CallbackHandle, &bFailedCall))
			{
				bool bFailedResult;
				// Retrieve the callback data from the request
				bool bSuccessCallResult = SteamUtilsPtr->GetAPICallResult(CallbackHandle, &CallbackResults, sizeof(CallbackResults), CallbackResults.k_iCallback, &bFailedResult);
				bWasSuccessful = bSuccessCallResult && !bFailedCall && !bFailedResult;
				if (bWasSuccessful)
				{
					FOnlineSessionSteamPtr SessionInt = StaticCastSharedPtr<FOnlineSessionSteam>(Subsystem->GetSessionInterface());
					check(SessionInt.IsValid());

					// Trigger the lobby data requests
					int32 NumLobbies = (int32)CallbackResults.m_nLobbiesMatching;
					for (int32 LobbyIdx = 0; LobbyIdx < NumLobbies; LobbyIdx++)
					{
						// We cannot matchmake any lobbies that we are currently in as this is unsupported by the platform
						FUniqueNetIdSteamRef LobbyID = FUniqueNetIdSteam::Create(SteamMatchmakingPtr->GetLobbyByIndex(LobbyIdx));
						if (!SessionInt->IsMemberOfLobby(*LobbyID))
						{
							LobbyIDs.Add(**LobbyID);
						}
					}
					FindLobbiesState = EFindLobbiesState::RequestLobbyData;
				}
				else
				{
					FindLobbiesState = EFindLobbiesState::Finished;
				}
			}
			break;
		}
	case EFindLobbiesState::RequestLobbyData:
		{
			bWasSuccessful = true;
			for (CSteamID LobbyId : LobbyIDs)
			{
				if (!SteamMatchmakingPtr->RequestLobbyData(LobbyId))
				{
					bWasSuccessful = false;
					FindLobbiesState = EFindLobbiesState::Finished;
					break;
				}

			}

			if (bWasSuccessful)
			{
				FindLobbiesState = EFindLobbiesState::WaitForRequestLobbyData;
			}

			break;
		}
	case EFindLobbiesState::WaitForRequestLobbyData:
		{
			FOnlineSessionSteamPtr SessionInt = StaticCastSharedPtr<FOnlineSessionSteam>(Subsystem->GetSessionInterface());

			float AsyncTimeout = ASYNC_TASK_TIMEOUT;
			GConfig->GetFloat(TEXT("OnlineSubsystemSteam"), TEXT("OnlineAsyncTaskSteamFindLobbiesBaseTimeout"), AsyncTimeout, GEngineIni);

			// Waiting for the lobby updates to fill in
			if (LobbyIDs.Num() == SessionInt->PendingSearchLobbyIds.Num())
			{
				FindLobbiesState = EFindLobbiesState::Finished;
			}
			// Fallback timeout in case we don't hear from Steam
			else if (GetElapsedTime() >= AsyncTimeout)
			{
				bWasSuccessful = false;
				FindLobbiesState = EFindLobbiesState::Finished;
			}
			break;
		}
	case EFindLobbiesState::Finished:
		{
			bIsComplete = true;
			break;
		}
	default:
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Unexpected state %d reached in FOnlineAsyncTaskSteamFindLobbiesBase::Tick, ending task."), (int32)FindLobbiesState);
			bWasSuccessful = false;
			FindLobbiesState = EFindLobbiesState::Finished;
			break;
		}
	}
}

/**
 * Give the async task a chance to marshal its data back to the game thread
 * Can only be called on the game thread by the async task manager
 */
void FOnlineAsyncTaskSteamFindLobbiesBase::Finalize()
{
	FOnlineSessionSteamPtr SessionInt = StaticCastSharedPtr<FOnlineSessionSteam>(Subsystem->GetSessionInterface());

	UE_LOG_ONLINE_SESSION(Log, TEXT("Found %d lobbies, finalizing the search"), SessionInt->PendingSearchLobbyIds.Num());

	if (bWasSuccessful)
	{
		// Parse any ready search results
		for (int32 LobbyIdx=0; LobbyIdx < SessionInt->PendingSearchLobbyIds.Num(); LobbyIdx++)
		{
			const FUniqueNetIdSteam& LobbyId = *SessionInt->PendingSearchLobbyIds[LobbyIdx];
			UE_LOG_ONLINE_SESSION(Log, TEXT("Search result %d: LobbyId=%s, LobbyId.IsValid()=%s, CSteamID(LobbyId).IsLobby()=%s"),
				LobbyIdx, *LobbyId.ToDebugString(), LobbyId.IsValid() ? TEXT("true") : TEXT("false"), (*LobbyId).IsLobby() ? TEXT("true") : TEXT("false")
				);
			if (LobbyId.IsValid() && (*LobbyId).IsLobby())
			{
				ParseSearchResult(LobbyId);
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Lobby %d is invalid (or not a lobby), skipping."), LobbyIdx);
			}
		}

		if (SearchSettings->SearchResults.Num() > 0)
		{
			// Allow game code to sort the servers
			SearchSettings->SortSearchResults();
		}
	}

	SearchSettings->SearchState = bWasSuccessful ? EOnlineAsyncTaskState::Done : EOnlineAsyncTaskState::Failed;
	if (SessionInt->CurrentSessionSearch.IsValid() && SearchSettings == SessionInt->CurrentSessionSearch)
	{
		SessionInt->CurrentSessionSearch = NULL;
	}

	SessionInt->PendingSearchLobbyIds.Empty();
}

/**
*	Get a human readable description of task
*/
FString FOnlineAsyncTaskSteamFindLobbies::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncTaskSteamFindLobbiesForFindSessions bWasSuccessful: %d NumResults: %d"), WasSuccessful(), CallbackResults.m_nLobbiesMatching);
}

/**
 *	Async task is given a chance to trigger it's delegates
 */
void FOnlineAsyncTaskSteamFindLobbies::TriggerDelegates()
{
	IOnlineSessionPtr SessionInt = Subsystem->GetSessionInterface();
	if (SessionInt.IsValid())
	{
		SessionInt->TriggerOnFindSessionsCompleteDelegates(bWasSuccessful);
	}
}

/**
*	Get a human readable description of task
*/
FString FOnlineAsyncTaskSteamFindLobbiesForInviteSession::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncTaskSteamFindLobbiesForInviteSession bWasSuccessful: %d Lobby ID: %llu"), WasSuccessful(), LobbyIDs[0].ConvertToUint64());
}

/**
*	Async task is given a chance to trigger it's delegates
*/
void FOnlineAsyncTaskSteamFindLobbiesForInviteSession::TriggerDelegates()
{
	if (bWasSuccessful && SearchSettings->SearchResults.Num() > 0)
	{
		OnFindLobbyCompleteWithNetIdDelegate.Broadcast(bWasSuccessful, LocalUserNum, FUniqueNetIdSteam::Create(SteamUser()->GetSteamID()), SearchSettings->SearchResults[0]);
	}
	else
	{
		FOnlineSessionSearchResult EmptyResult;
		OnFindLobbyCompleteWithNetIdDelegate.Broadcast(bWasSuccessful, LocalUserNum, FUniqueNetIdSteam::Create(SteamUser()->GetSteamID()), EmptyResult);
	}
}

/**
*	Get a human readable description of task
*/
FString FOnlineAsyncTaskSteamFindLobbiesForFriendSession::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncTaskSteamFindLobbiesForFriendSession bWasSuccessful: %d Lobby ID: %llu"), WasSuccessful(), LobbyIDs[0].ConvertToUint64());
}

/**
*	Async task is given a chance to trigger it's delegates
*/
void FOnlineAsyncTaskSteamFindLobbiesForFriendSession::TriggerDelegates()
{
	if (bWasSuccessful && SearchSettings->SearchResults.Num() > 0)
	{
		OnFindFriendSessionCompleteDelegate.Broadcast(LocalUserNum, bWasSuccessful, SearchSettings->SearchResults);
	}
	else
	{
		TArray<FOnlineSessionSearchResult> EmptyResult;
		OnFindFriendSessionCompleteDelegate.Broadcast(LocalUserNum, bWasSuccessful, EmptyResult);
	}
}

/**
 *	Get a human readable description of task
 */
FString FOnlineAsyncEventSteamLobbyInviteAccepted::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncEventSteamLobbyInviteAccepted LobbyId: %s Friend: %s"), 
		*LobbyId->ToDebugString(),
		*FriendId->ToDebugString());	
}

/**
 * Give the async task a chance to marshal its data back to the game thread
 * Can only be called on the game thread by the async task manager
 */
void FOnlineAsyncEventSteamLobbyInviteAccepted::Finalize()
{
	FOnlineSessionSteamPtr SessionInt = StaticCastSharedPtr<FOnlineSessionSteam>(Subsystem->GetSessionInterface());
	if (SessionInt.IsValid() && !SessionInt->CurrentSessionSearch.IsValid())
	{
		// Create a search settings object 
		SessionInt->CurrentSessionSearch = MakeShareable(new FOnlineSessionSearch());
		SessionInt->CurrentSessionSearch->SearchState = EOnlineAsyncTaskState::InProgress;

		FOnlineAsyncTaskSteamFindLobbiesForInviteSession* NewTask = new FOnlineAsyncTaskSteamFindLobbiesForInviteSession(Subsystem, *LobbyId, SessionInt->CurrentSessionSearch, LocalUserNum, SessionInt->OnSessionUserInviteAcceptedDelegates);
		Subsystem->QueueAsyncTask(NewTask);
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Invalid session or search already in progress when accepting invite.  Ignoring invite request."));
	}
}
