// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemSteamTypes.h"
#include "OnlineSessionSettings.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineAsyncTaskManagerSteam.h"

class FOnlineSubsystemSteam;

/** 
 *  Async task for creating a Steam backend lobby as host and defining the proper settings
 */
class FOnlineAsyncTaskSteamCreateLobby : public FOnlineAsyncTaskSteam
{
private:

	/** Has this request been started */
	bool bInit;
	/** Name of session being created */
	FName SessionName;
	/** Type of lobby to create */
	ELobbyType LobbyType;
	/** Max number of players allowed */
	int32 MaxLobbyMembers;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamCreateLobby() : 
		bInit(false),
		SessionName(NAME_None),
		LobbyType(k_ELobbyTypePublic),
		MaxLobbyMembers(0)
	{
	}

PACKAGE_SCOPE:

	/** Lobby created callback data */
	LobbyCreated_t CallbackResults;

public:

	FOnlineAsyncTaskSteamCreateLobby(class FOnlineSubsystemSteam* InSubsystem, FName InSessionName, ELobbyType InLobbyType, int32 InMaxLobbyMembers) :
		FOnlineAsyncTaskSteam(InSubsystem, k_uAPICallInvalid),
		bInit(false),
		SessionName(InSessionName),
		LobbyType(InLobbyType),
		MaxLobbyMembers(InMaxLobbyMembers)
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override;

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override;

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override;
	
	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override;
};

/** 
 *  Async task to update a single Steam lobby
 */
class FOnlineAsyncTaskSteamUpdateLobby : public FOnlineAsyncTaskSteam
{
private:

	/** Name of session being created */
	FName SessionName;
	/** New session settings to apply */
	FOnlineSessionSettings NewSessionSettings;
	/** Should the online platform refresh as well */
	bool bUpdateOnlineData;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamUpdateLobby() : 
		SessionName(NAME_None),
		bUpdateOnlineData(false)
	{
	}

public:

	/** Constructor */
	FOnlineAsyncTaskSteamUpdateLobby(class FOnlineSubsystemSteam* InSubsystem, FName InSessionName, bool bInUpdateOnlineData, const FOnlineSessionSettings& InNewSessionSettings) :
		FOnlineAsyncTaskSteam(InSubsystem, k_uAPICallInvalid),
		SessionName(InSessionName),
		NewSessionSettings(InNewSessionSettings),
		bUpdateOnlineData(bInUpdateOnlineData)
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override;

	/**
	* Give the async task time to do its work
	* Can only be called on the async task manager thread
	*/
	virtual void Tick() override;
	
	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override;
};

/** 
 *  Async task to join a single Steam lobby
 */
class FOnlineAsyncTaskSteamJoinLobby : public FOnlineAsyncTaskSteam
{
private:

	/** Has this request been started */
	bool bInit;
	/** Name of session being created */
	FName SessionName;
	/** Lobby to join */
	FUniqueNetIdSteamRef LobbyId;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamJoinLobby() = delete;

PACKAGE_SCOPE:

	/** Join request callback data */
	LobbyEnter_t CallbackResults;

public:

	/** Constructor */
	FOnlineAsyncTaskSteamJoinLobby(class FOnlineSubsystemSteam* InSubsystem, FName InSessionName, const FUniqueNetIdSteam& InLobbyId) :
		FOnlineAsyncTaskSteam(InSubsystem, k_uAPICallInvalid),
		bInit(false),
		SessionName(InSessionName),
		LobbyId(InLobbyId.AsShared())
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override;

	/**
	* Give the async task time to do its work
	* Can only be called on the async task manager thread
	*/
	virtual void Tick() override;

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override;
	
	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override;
};

/** 
 *  Async task for leaving a single lobby
 */
class FOnlineAsyncTaskSteamLeaveLobby : public FOnlineAsyncTaskSteam
{
private:

	/** Name of session lobby */
	FName SessionName;
	/** LobbyId to end */
	FUniqueNetIdSteamRef LobbyId;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamLeaveLobby() = delete;

public:

	FOnlineAsyncTaskSteamLeaveLobby(class FOnlineSubsystemSteam* InSubsystem, FName InSessionName, const FUniqueNetIdSteam& InLobbyId) :
		FOnlineAsyncTaskSteam(InSubsystem, k_uAPICallInvalid),
		SessionName(InSessionName),
		LobbyId(InLobbyId.AsShared())
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override;

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override;
};

/** 
 *  Async task for any search query to find Steam lobbies based on search criteria
 */
class FOnlineAsyncTaskSteamFindLobbiesBase : public FOnlineAsyncTaskSteam
{
private:
	/** Cached instance of Steam interface */
	ISteamMatchmaking* SteamMatchmakingPtr;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamFindLobbiesBase() :
		SteamMatchmakingPtr(NULL)
	{	
	}

	/**
	 *	Create and trigger the lobby query from the defined search settings 
	 */
	virtual void CreateQuery();

PACKAGE_SCOPE:

	enum class EFindLobbiesState : uint8
	{
		Init,
		RequestLobbyList,
		RequestLobbyData,
		WaitForRequestLobbyData,
		Finished
	};

	/** Search settings specified for the query */
	TSharedPtr<class FOnlineSessionSearch> SearchSettings;

	EFindLobbiesState FindLobbiesState;
	/** Lobby search callback data */
	LobbyMatchList_t CallbackResults;

	TArray<CSteamID> LobbyIDs;

public:

	/** Constructor */
	FOnlineAsyncTaskSteamFindLobbiesBase(class FOnlineSubsystemSteam* InSubsystem, const TSharedPtr<FOnlineSessionSearch>& InSearchSettings) :
		FOnlineAsyncTaskSteam(InSubsystem, k_uAPICallInvalid),
		SteamMatchmakingPtr(SteamMatchmaking()),
		SearchSettings(InSearchSettings),
		FindLobbiesState(EFindLobbiesState::Init)
	{
	}

	/**
	 *	Create a search result from a specified lobby id 
	 *
	 * @param LobbyId lobby to create the search result for
	 */
	void ParseSearchResult(const FUniqueNetIdSteam& LobbyId);

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override;

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override;
};

/**
*  Async task for any search query to find Steam lobbies based on search criteria
*/
class FOnlineAsyncTaskSteamFindLobbies : public FOnlineAsyncTaskSteamFindLobbiesBase
{

public:

	/** Constructor */
	FOnlineAsyncTaskSteamFindLobbies(class FOnlineSubsystemSteam* InSubsystem, const TSharedPtr<FOnlineSessionSearch>& InSearchSettings) :
		FOnlineAsyncTaskSteamFindLobbiesBase(InSubsystem, InSearchSettings)
	{
	}

	/**
	*	Get a human readable description of task
	*/
	virtual FString ToString() const override;

	/**
	*	Async task is given a chance to trigger it's delegates
	*/
	virtual void TriggerDelegates() override;
};

DECLARE_MULTICAST_DELEGATE_FourParams(FOnAsyncFindLobbyCompleteWithNetId, const bool, const int32, FUniqueNetIdPtr, const class FOnlineSessionSearchResult&);
typedef FOnAsyncFindLobbyCompleteWithNetId::FDelegate FOnAsyncFindLobbyCompleteDelegateWithNetId;

class FOnlineAsyncTaskSteamFindLobbiesForInviteSession : public FOnlineAsyncTaskSteamFindLobbiesBase
{
public:
	/** Constructor */
	FOnlineAsyncTaskSteamFindLobbiesForInviteSession(class FOnlineSubsystemSteam* InSubsystem, const FUniqueNetIdSteam& InLobbyId, const TSharedPtr<FOnlineSessionSearch>& InSearchSettings, int32 InLocalUserNum, const FOnAsyncFindLobbyCompleteWithNetId& InOnFindLobbyCompleteDelegates) :
		FOnlineAsyncTaskSteamFindLobbiesBase(InSubsystem, InSearchSettings),
		LocalUserNum(InLocalUserNum),
		OnFindLobbyCompleteWithNetIdDelegate(InOnFindLobbyCompleteDelegates)
	{
		LobbyIDs.Add(CSteamID(*(uint64*)InLobbyId.GetBytes()));
		FindLobbiesState = FOnlineAsyncTaskSteamFindLobbiesBase::EFindLobbiesState::RequestLobbyData;
	}

	/**
	*	Get a human readable description of task
	*/
	virtual FString ToString() const override;

	/**
	*	Async task is given a chance to trigger it's delegates
	*/
	virtual void TriggerDelegates() override;

private:

	/** User initiating the request */
	int32 LocalUserNum;

	FOnAsyncFindLobbyCompleteWithNetId OnFindLobbyCompleteWithNetIdDelegate;
};

class FOnlineAsyncTaskSteamFindLobbiesForFriendSession : public FOnlineAsyncTaskSteamFindLobbiesBase
{
public:
	/** Constructor */
	FOnlineAsyncTaskSteamFindLobbiesForFriendSession(class FOnlineSubsystemSteam* InSubsystem, const FUniqueNetIdSteam& InLobbyId, const TSharedPtr<FOnlineSessionSearch>& InSearchSettings, int32 InLocalUserNum, const FOnFindFriendSessionComplete& InOnFindFriendSessionCompleteDelegate) :
		FOnlineAsyncTaskSteamFindLobbiesBase(InSubsystem, InSearchSettings),
		LocalUserNum(InLocalUserNum),
		OnFindFriendSessionCompleteDelegate(InOnFindFriendSessionCompleteDelegate)
	{
		LobbyIDs.Add(CSteamID(*(uint64*)InLobbyId.GetBytes()));
		FindLobbiesState = FOnlineAsyncTaskSteamFindLobbiesBase::EFindLobbiesState::RequestLobbyData;
	}

	/**
	*	Get a human readable description of task
	*/
	virtual FString ToString() const override;

	/**
	*	Async task is given a chance to trigger it's delegates
	*/
	virtual void TriggerDelegates() override;

private:

	/** User initiating the request */
	int32 LocalUserNum;

	FOnFindFriendSessionComplete OnFindFriendSessionCompleteDelegate;
};

/**
 *	Turns a friends accepted invite request into a valid search result (lobby version)
 */
class FOnlineAsyncEventSteamLobbyInviteAccepted : public FOnlineAsyncEvent<FOnlineSubsystemSteam>
{
	/** Friend that invited */
	FUniqueNetIdSteamRef FriendId;
	/** Lobby to go to */
	FUniqueNetIdSteamRef LobbyId;
	/** User initiating the request */
	int32 LocalUserNum;

	/** Hidden on purpose */
	FOnlineAsyncEventSteamLobbyInviteAccepted() = delete;

public:
	FOnlineAsyncEventSteamLobbyInviteAccepted(FOnlineSubsystemSteam* InSubsystem, const FUniqueNetIdSteam& InFriendId, const FUniqueNetIdSteam& InLobbyId) :
		FOnlineAsyncEvent(InSubsystem),
		FriendId(InFriendId.AsShared()),
		LobbyId(InLobbyId.AsShared()),
		LocalUserNum(0)
	{
	}
	
	virtual ~FOnlineAsyncEventSteamLobbyInviteAccepted() 
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override;

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override;
};

/**
 *	Generate the proper lobby type from session settings
 *
 * @param SessionSettings current settings for the session
 *
 * @return type of lobby to generate, defaulting to private if not advertising and public otherwise
 */
ELobbyType BuildLobbyType(FOnlineSessionSettings* SessionSettings);

/**
 *	Populate an FSession data structure from the data stored with a lobby
 * Expects a certain number of keys to be present otherwise this will fail
 *
 * @param SteamSubsystem the online subsystem to use
 * @param LobbyId the Steam lobby to fill data from
 * @param Session empty session structure to fill in
 * @param SearchData if this is provided and the Steam session supports SteamSockets, the ping for the lobby will be written into the search result data.
 *
 * @return true if successful, false otherwise
 */
bool FillSessionFromLobbyData(FOnlineSubsystemSteam* SteamSubsystem, const FUniqueNetIdSteam& LobbyId, class FOnlineSession& Session, FOnlineSessionSearchResult* SearchData = nullptr);

/**
 *	Populate an FSession data structure from the data stored with the members of the lobby
 *
 * @param LobbyId the Steam lobby to fill data from
 * @param Session session structure to fill in
 *
 * @return true if successful, false otherwise
 */
bool FillMembersFromLobbyData(const FUniqueNetIdSteam& LobbyId, class FNamedOnlineSession& Session);
