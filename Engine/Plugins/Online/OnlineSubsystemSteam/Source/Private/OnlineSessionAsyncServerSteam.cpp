// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSessionAsyncServerSteam.h"
#include "Misc/ConfigCacheIni.h"
#include "Online/OnlineSessionNames.h"
#include "IPAddressSteam.h"
#include "SteamSessionKeys.h"
#include "SteamUtilities.h"
#include "OnlineAuthInterfaceSteam.h"
#include <steam/steamuniverse.h>

#if WITH_ENGINE
#include "GameFramework/GameStateBase.h"
#include "GameFramework/PlayerState.h"
#include "OnlineSubsystemUtils.h"
#include "SocketSubsystem.h"
#endif //WITH_ENGINE


/** Turn on Steam filter generation output */
#define DEBUG_STEAM_FILTERS 1

/* Server values needed to advertise with Steam (NOTE: Steam expects UTF8)
 *
 * Specify these in your Target.cs files for your project.
 * Make sure to escape the strings!
 * If these do not match the same values on the Steam partner backend, 
 * matchmaking will not work for your dedicated server
 */
#ifndef UE_PROJECT_STEAMPRODUCTNAME
#define UE_PROJECT_STEAMPRODUCTNAME "unrealdk"
#endif //UE_PROJECT_STEAMPRODUCTNAME

#ifndef UE_PROJECT_STEAMGAMEDIR
#define UE_PROJECT_STEAMGAMEDIR "unrealtest"
#endif // UE_PROJECT_STEAMGAMEDIR

#ifndef UE_PROJECT_STEAMGAMEDESC
#define UE_PROJECT_STEAMGAMEDESC "Unreal Test!"
#endif // UE_PROJECT_STEAMGAMEDESC

/**
 * Get the engine unique build id as Steam key
 *
 * @return BuildUniqueId as an FString
 */
FString GetBuildIdAsSteamKey(const FOnlineSessionSettings& SessionSettings)
{
	return FString::Printf(TEXT("%s:%d"), UTF8_TO_TCHAR(STEAMKEY_BUILDUNIQUEID), SessionSettings.BuildUniqueId);
}

/**
 * Get the session flags bitfield as an FString 
 *
 * @param SessionSettings settings to transform into string
 *
 * @return SessionFlags as an FString
 */
FString GetSessionFlagsAsString(const FOnlineSessionSettings& SessionSettings)
{
	int32 BitShift = 0;
	int32 SessionFlags = 0;
	// Some of this is redundant but included for completeness (bAntiCheatProtected, etc)
	SessionFlags |= (SessionSettings.bShouldAdvertise ? 1 : 0) << BitShift++;
	SessionFlags |= (SessionSettings.bAllowJoinInProgress ? 1 : 0) << BitShift++;
	SessionFlags |= (SessionSettings.bIsLANMatch ? 1 : 0) << BitShift++;
	SessionFlags |= (SessionSettings.bIsDedicated ? 1 : 0) << BitShift++;
	SessionFlags |= (SessionSettings.bUsesStats ? 1 : 0) << BitShift++;
	SessionFlags |= (SessionSettings.bAllowInvites ? 1 : 0) << BitShift++;
	SessionFlags |= (SessionSettings.bUsesPresence ? 1 : 0) << BitShift++;
	SessionFlags |= (SessionSettings.bAllowJoinViaPresence ? 1 : 0) << BitShift++;
	SessionFlags |= (SessionSettings.bAllowJoinViaPresenceFriendsOnly ? 1 : 0) << BitShift++;
	SessionFlags |= (SessionSettings.bAntiCheatProtected ? 1 : 0) << BitShift++;

	return FString::FromInt(SessionFlags);
}

/**
 *	Get all relevant FOnlineSessionSettings data as a series of Key,Value pairs
 *
 * @param SessionSettings session settings to get key, value pairs from
 * @param KeyValuePairs key value pair structure to add to
 */
void GetServerKeyValuePairsFromSessionSettings(const FOnlineSessionSettings& SessionSettings, FSteamSessionKeyValuePairs& KeyValuePairs, EOnlineDataAdvertisementType::Type AdvertisementType)
{
	//KeyValuePairs.Add(STEAMKEY_NUMPUBLICCONNECTIONS, FString::FromInt(SessionSettings.NumPublicConnections));
	//KeyValuePairs.Add(STEAMKEY_NUMPRIVATECONNECTIONS, FString::FromInt(SessionSettings.NumPrivateConnections));
	
	FString KeyStr;
	for (FSessionSettings::TConstIterator It(SessionSettings.Settings); It; ++It)
	{
		FName Key = It.Key();
		const FOnlineSessionSetting& Setting = It.Value();
		if (Setting.AdvertisementType == AdvertisementType)
		{
			if (SessionKeyToSteamKey(Key, Setting.Data, KeyStr))
			{
				FString SettingStr = Setting.Data.ToString();
				if (!SettingStr.IsEmpty())
				{
					KeyValuePairs.Add(KeyStr, SettingStr);
				}
				else
				{
					UE_LOG_ONLINE_SESSION(Warning, TEXT("Empty session setting %s %s of type %s"), *Key.ToString(), *Setting.ToString(), EOnlineKeyValuePairDataType::ToString(Setting.Data.GetType()));
				}
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Unsupported session setting %s %s of type %s"), *Key.ToString(), *Setting.ToString(), EOnlineKeyValuePairDataType::ToString(Setting.Data.GetType()));
			}
		}
	}
}

/**
 *	Get all relevant FSessionInfoSteam data as a series of Key,Value pairs
 *
 * @param SessionInfo session info to get key, value pairs from
 * @param KeyValuePairs key value pair structure to add to
 */
void GetServerKeyValuePairsFromSessionInfo(FOnlineSessionInfoSteam* SessionInfo, FSteamSessionKeyValuePairs& KeyValuePairs)
{
	FOnlineSubsystemSteam* SteamSubsystem = static_cast<FOnlineSubsystemSteam*>(IOnlineSubsystem::Get(STEAM_SUBSYSTEM));
	FSteamConnectionMethod MethodUsed = SessionInfo->ConnectionMethod;

	// We should not enter this check here if we're using LAN.
	if (SteamSubsystem && GConfig && MethodUsed == FSteamConnectionMethod::None)
	{
		bool bUseRelays = false;
		bool bIsDefaultSubsystem = false;
		GConfig->GetBool(TEXT("OnlineSubsystemSteam"), TEXT("bAllowP2PPacketRelay"), bUseRelays, GEngineIni);
		// If this config specifically does not exist, then set it to true
		if (!GConfig->GetBool(TEXT("OnlineSubsystemSteam"), TEXT("bUseSteamNetworking"), bIsDefaultSubsystem, GEngineIni))
		{
			bIsDefaultSubsystem = true;
		}

		// Using Steam sockets. This checks if the SteamSockets plugin is enabled
		if (!SteamSubsystem->IsUsingSteamNetworking())
		{
			if (!bUseRelays)
			{
				MethodUsed = FSteamConnectionMethod::Direct;
			}
			else
			{
				MethodUsed = FSteamConnectionMethod::P2P;
			}
		}
		else
		{
			// Determine if we are a default. If so, we're using SteamNetworking P2P.
			if (bIsDefaultSubsystem)
			{
				MethodUsed = FSteamConnectionMethod::P2P;
			}
			else
			{
				MethodUsed = FSteamConnectionMethod::Direct;
			}
		}

		// Set this if it hasn't been set yet.
		SessionInfo->ConnectionMethod = MethodUsed;
	}

	KeyValuePairs.Add(STEAMKEY_CONNECTIONMETHOD, FString::Printf(TEXT("%s"), *LexToString(MethodUsed)));

	if (SessionInfo->SteamP2PAddr.IsValid())
	{
		KeyValuePairs.Add(STEAMKEY_P2PADDR, SessionInfo->SteamP2PAddr->ToString(false));
		KeyValuePairs.Add(STEAMKEY_P2PPORT, FString::FromInt(SessionInfo->SteamP2PAddr->GetPort()));
	}
}

/**
 *	Get all relevant FSession data as a series of Key,Value pairs
 *
 * @param Session session data to get key, value pairs from
 * @param KeyValuePairs key value pair structure to add to
 */
void GetServerKeyValuePairsFromSession(const FOnlineSession* Session, FSteamSessionKeyValuePairs& KeyValuePairs)
{
	FUniqueNetIdSteam* SteamId = (FUniqueNetIdSteam*)(Session->OwningUserId.Get());
	FString OwningUserIdStr = SteamId->ToString();
	KeyValuePairs.Add(STEAMKEY_OWNINGUSERID, OwningUserIdStr);
	KeyValuePairs.Add(STEAMKEY_OWNINGUSERNAME, Session->OwningUserName);
	//KeyValuePairs.Add(STEAMKEY_NUMOPENPRIVATECONNECTIONS, FString::FromInt(Session->NumOpenPrivateConnections));
	//KeyValuePairs.Add(STEAMKEY_NUMOPENPUBLICCONNECTIONS, FString::FromInt(Session->NumOpenPublicConnections));
}

/** 
 *  Update the backend with the currently defined settings
 *
 * @param World current running world instance
 * @param SessionName name of session to update published settings
 */
void UpdatePublishedSettings(UWorld* World, FNamedOnlineSession* Session)
{
	ISteamGameServer* SteamGameServerPtr = SteamGameServer();
	check(SteamGameServerPtr);

	// Copy the current settings so we can remove the ones used for well defined search parameters
	FOnlineSessionSettings TempSessionSettings = Session->SessionSettings;

	// Server name
	FString ServerName = Session->OwningUserName;
	SteamGameServerPtr->SetServerName(TCHAR_TO_UTF8(*ServerName));

	// Max user slots reported
	int32 NumTotalSlots = Session->SessionSettings.NumPublicConnections + Session->SessionSettings.NumPrivateConnections;
	SteamGameServerPtr->SetMaxPlayerCount(NumTotalSlots);

	// Region setting
	FString Region(TEXT(""));
	SteamGameServerPtr->SetRegion(TCHAR_TO_UTF8(*Region));

	// @TODO ONLINE Password protected or not
	SteamGameServerPtr->SetPasswordProtected(false);

	// Map name
	FString MapName;
	if (TempSessionSettings.Get(SETTING_MAPNAME, MapName) && !MapName.IsEmpty())
	{
		SteamGameServerPtr->SetMapName(TCHAR_TO_UTF8(*MapName));
	}
	TempSessionSettings.Remove(SETTING_MAPNAME);

	// Bot Count
	int32 BotCount = 0;
	if (TempSessionSettings.Get(SETTING_NUMBOTS, BotCount))
	{
		SteamGameServerPtr->SetBotPlayerCount(BotCount);
	}
	TempSessionSettings.Remove(SETTING_NUMBOTS);

	// Update all the players names/scores
#if WITH_ENGINE
	if (World)
	{
		AGameStateBase const* const GameState = World->GetGameState();
		if (GameState)
		{
			for (int32 PlayerIdx=0; PlayerIdx < GameState->PlayerArray.Num(); PlayerIdx++)
			{
				APlayerState const* const PlayerState = GameState->PlayerArray[PlayerIdx];
				if (PlayerState && PlayerState->GetUniqueId().IsValid())
				{
					CSteamID SteamId(*(uint64*)PlayerState->GetUniqueId()->GetBytes());
					SteamGameServerPtr->BUpdateUserData(SteamId, TCHAR_TO_UTF8(*PlayerState->GetPlayerName()), PlayerState->GetScore());
				}
			}
		}
	}
#endif //WITH_ENGINE

	// Get the advertised session settings out as Steam key/value pairs
	FSteamSessionKeyValuePairs ViaOnlineServicePairs;
	GetServerKeyValuePairsFromSession(Session, ViaOnlineServicePairs);

	// Grab the session flags
	ViaOnlineServicePairs.Add(STEAMKEY_SESSIONFLAGS, GetSessionFlagsAsString(Session->SessionSettings));

	// Start the game tags with the build id so search results can early out
	FString GameTagsString = GetBuildIdAsSteamKey(Session->SessionSettings);

	GetServerKeyValuePairsFromSessionSettings(TempSessionSettings, ViaOnlineServicePairs, EOnlineDataAdvertisementType::ViaOnlineService);

	FSteamSessionKeyValuePairs ViaPingPairs;
	GetServerKeyValuePairsFromSessionSettings(TempSessionSettings, ViaPingPairs, EOnlineDataAdvertisementType::ViaPingOnly);

	FSteamSessionKeyValuePairs ViaOnlineServicePingPairs;
	GetServerKeyValuePairsFromSessionSettings(TempSessionSettings, ViaOnlineServicePingPairs, EOnlineDataAdvertisementType::ViaOnlineServiceAndPing);

	// Generate the tag data first, due to its small size
	for (FSteamSessionKeyValuePairs::TConstIterator It(ViaOnlineServicePairs); It; ++It)
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("Server List Game Tags (%s, %s)"), *It.Key(), *It.Value());
		FString NewKey = FString::Printf(TEXT(",%s:%s"), *It.Key(), *It.Value());
		if (GameTagsString.Len() + NewKey.Len() < k_cbMaxGameServerTags)
		{
			GameTagsString += NewKey;
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Server setting %s overflows Steam SetGameTags call"), *NewKey);
		}
	}

	// Push in the other tags from session info (these are dropped from the tag queries for space reasons)
	if (Session->SessionInfo.IsValid())
	{
		FOnlineSessionInfoSteam* SessionInfo = (FOnlineSessionInfoSteam*)(Session->SessionInfo.Get());
		GetServerKeyValuePairsFromSessionInfo(SessionInfo, ViaOnlineServicePairs);
	}

	// Append both advertising sets with the dual advertisement filters
	ViaOnlineServicePairs.Append(ViaOnlineServicePingPairs);

	// Create the properly formatted Steam string (ie key:value,key:value,key) for GameData
	FString GameDataString;
	for (FSteamSessionKeyValuePairs::TConstIterator It(ViaOnlineServicePairs); It; ++It)
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("Server List Game Data (%s, %s)"), *It.Key(), *It.Value());
		FString NewKey = FString::Printf(TEXT("%s:%s"), *It.Key(), *It.Value());

		if (GameDataString.Len() + NewKey.Len() < k_cbMaxGameServerGameData)
		{
			GameDataString += ((GameDataString.Len() > 0) ? "," : "") + NewKey;
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Server setting %s overflows Steam SetGameData call"), *NewKey);
		}
	}

	// Small and searchable game tags (returned in initial server query structure)
	if (GameTagsString.Len() > 0 && GameTagsString.Len() < k_cbMaxGameServerTags)
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("SetGameTags(%s)"), *GameTagsString);
		SteamGameServerPtr->SetGameTags(TCHAR_TO_UTF8(*GameTagsString));
	}

	// Large and searchable game data (never returned, used to filter things on the server list)
	if (GameDataString.Len() > 0 && GameDataString.Len() < k_cbMaxGameServerGameData)
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("SetGameData(%s)"), *GameDataString);
		SteamGameServerPtr->SetGameData(TCHAR_TO_UTF8(*GameDataString));
	}

	// Set the advertised filter keys (these can not be filtered at server list level, only client side via talking to the server)
	SteamGameServerPtr->ClearAllKeyValues();

	// Push all the filters in for the KeyValue data.
	ViaOnlineServicePairs.Append(ViaPingPairs);

	// Key value pairs sent as rules (requires secondary RulesRequest call)
	for (FSteamSessionKeyValuePairs::TConstIterator AuxKeyIt(ViaOnlineServicePairs); AuxKeyIt; ++AuxKeyIt)
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("Pushing Server KVData (%s, %s)"), *AuxKeyIt.Key(), *AuxKeyIt.Value());
		SteamGameServerPtr->SetKeyValue(TCHAR_TO_UTF8(*AuxKeyIt.Key()), TCHAR_TO_UTF8(*AuxKeyIt.Value()));
	}
}

/**
 *	Get a human readable description of task
 */
FString FOnlineAsyncTaskSteamCreateServer::ToString() const 
{
	return FString::Printf(TEXT("FOnlineAsyncTaskSteamCreateServer bWasSuccessful: %d"), WasSuccessful());
}

/**
 * Give the async task time to do its work
 * Can only be called on the async task manager thread
 */
void FOnlineAsyncTaskSteamCreateServer::Tick() 
{
	FOnlineSessionSteamPtr SessionInt = StaticCastSharedPtr<FOnlineSessionSteam>(Subsystem->GetSessionInterface());
	if (!bInit)
	{
		ISteamGameServer* SteamGameServerPtr = SteamGameServer();
		FNamedOnlineSession* Session = (SessionInt.IsValid()) ? SessionInt->GetNamedSession(SessionName) : nullptr;
		if (Session != nullptr && SteamGameServerPtr != nullptr)
		{
			bool bWantsDedicated = Session->SessionSettings.bIsDedicated;
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("Starting Steam game server. Dedicated? %d Game Dir is: %s Product Name is: %s\nGame Desc is: %s"),
				bWantsDedicated,
				ANSI_TO_TCHAR((UE_PROJECT_STEAMGAMEDIR)),
				ANSI_TO_TCHAR((UE_PROJECT_STEAMPRODUCTNAME)),
				ANSI_TO_TCHAR((UE_PROJECT_STEAMGAMEDESC)));

			SteamGameServerPtr->SetModDir(UE_PROJECT_STEAMGAMEDIR);
			SteamGameServerPtr->SetProduct(UE_PROJECT_STEAMPRODUCTNAME);
			SteamGameServerPtr->SetGameDescription(UE_PROJECT_STEAMGAMEDESC);
			SteamGameServerPtr->SetDedicatedServer(bWantsDedicated);

			if (!SteamGameServerPtr->BLoggedOn())
			{
				// Login the server with Steam
				SteamGameServerPtr->LogOnAnonymous();
			}

			// Setup advertisement and force the initial update
			//SteamGameServerPtr->SetHeartbeatInterval(-1);      // deprecated in 1.53
			SteamGameServerPtr->SetAdvertiseServerActive(true);  // renamed EnableHeartbeats to SetAdvertiseServerActive in 1.53
			// SteamGameServerPtr->ForceHeartbeat();             // deprecated in 1.53

			bInit = true;
		}
	}

	// Wait for the connection and policy response callbacks
	if (bInit && SessionInt->bSteamworksGameServerConnected && SessionInt->GameServerSteamId->IsValid() && SessionInt->bPolicyResponseReceived)
	{
		bIsComplete = true;
		bWasSuccessful = true;
	}
	else
	{
		float AsyncTimeout = ASYNC_TASK_TIMEOUT;
		GConfig->GetFloat(TEXT("OnlineSubsystemSteam"), TEXT("OnlineAsyncTaskSteamCreateServerTimeout"), AsyncTimeout, GEngineIni);
		
		// Fallback timeout in case we don't hear from Steam
		if (GetElapsedTime() >= AsyncTimeout)
		{
			bIsComplete = true;
			bWasSuccessful = false;
		}
	}
}

/**
 * Give the async task a chance to marshal its data back to the game thread
 * Can only be called on the game thread by the async task manager
 */
void FOnlineAsyncTaskSteamCreateServer::Finalize() 	
{
	FOnlineSessionSteamPtr SessionInt = StaticCastSharedPtr<FOnlineSessionSteam>(Subsystem->GetSessionInterface());
	if (bWasSuccessful)
	{
		FNamedOnlineSession* Session = SessionInt->GetNamedSession(SessionName);
		if (Session)
		{
			// Setup the host session info
			FOnlineSessionInfoSteam* NewSessionInfo = new FOnlineSessionInfoSteam(ESteamSession::AdvertisedSessionHost, *SessionInt->GameServerSteamId);
			NewSessionInfo->Init();

			ISteamGameServer* SteamGameServerPtr = SteamGameServer();
			check(SteamGameServerPtr);

#if WITH_ENGINE
			// Create the proper Steam P2P address for this machine
			NewSessionInfo->SteamP2PAddr = ISocketSubsystem::Get()->GetLocalBindAddr(*GLog);
			NewSessionInfo->SteamP2PAddr->SetPort(Subsystem->GetGameServerGamePort());
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("Server SteamP2P IP: %s"), *NewSessionInfo->SteamP2PAddr->ToString(true));

			// Create the proper ip address for this server
			NewSessionInfo->HostAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();
			NewSessionInfo->HostAddr->SetIp(SteamGameServerPtr->GetPublicIP().m_unIPv4);
			NewSessionInfo->HostAddr->SetPort(Subsystem->GetGameServerGamePort());
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("Server IP: %s"), *NewSessionInfo->HostAddr->ToString(true));
#endif //WITH_ENGINE

			if (!Session->OwningUserId.IsValid())
			{
				check(Session->SessionSettings.bIsDedicated);
				// Associate the dedicated server anonymous login as the owning user
				Session->OwningUserId = SessionInt->GameServerSteamId;

				// Figure if we need to override the server name here
				FString CustomDedicatedServerName = SessionInt->GetCustomDedicatedServerName();
				Session->OwningUserName = (!CustomDedicatedServerName.IsEmpty()) ? CustomDedicatedServerName : Session->OwningUserId->ToString();
			}

			bool bShouldUseAdvertise = true;
			FOnlineAuthSteamPtr SteamAuth = Subsystem->GetAuthInterface();
			if (SteamAuth.IsValid())
			{
				// Do not use the old advertisegame function because SteamAuth will handle it for us
				bShouldUseAdvertise = !SteamAuth->IsSessionAuthEnabled();
			}
			
			Session->SessionInfo = MakeShareable(NewSessionInfo);
			Session->SessionSettings.bAntiCheatProtected = SteamGameServerPtr->BSecure() != 0 ? true : false;

			Session->SessionState = EOnlineSessionState::Pending;

#if WITH_ENGINE
			UWorld* World = GetWorldForOnline(Subsystem->GetInstanceName());
			UpdatePublishedSettings(World, Session);
#endif //WITH_ENGINE

			SessionInt->RegisterLocalPlayers(Session);
			
			if (SteamUser() && bShouldUseAdvertise)
			{
				UE_LOG_ONLINE(Warning, TEXT("AUTH: CreateServerSteam is calling the depricated AdvertiseGame call"));
				SteamUser()->AdvertiseGame(k_steamIDNonSteamGS, SteamGameServerPtr->GetPublicIP().m_unIPv4, Subsystem->GetGameServerGamePort());
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("No session %s found to update with Steam backend"), *SessionName.ToString());
		}
	}
	else
	{
		SessionInt->RemoveNamedSession(SessionName);
	}
}

/**
 *	Async task is given a chance to trigger it's delegates
 */
void FOnlineAsyncTaskSteamCreateServer::TriggerDelegates() 
{
	IOnlineSessionPtr SessionInt = Subsystem->GetSessionInterface();
	if (SessionInt.IsValid())
	{
		SessionInt->TriggerOnCreateSessionCompleteDelegates(SessionName, bWasSuccessful);
	}
}

/**
 *	Get a human readable description of task
 */
FString FOnlineAsyncTaskSteamUpdateServer::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncTaskSteamUpdateServer bWasSuccessful: %d Session: %s"),
		WasSuccessful(),
		*SessionName.ToString());
}

/**
 * Give the async task time to do its work
 * Can only be called on the async task manager thread
 */
void FOnlineAsyncTaskSteamUpdateServer::Tick()
{
	FOnlineSessionSteamPtr SessionInt = StaticCastSharedPtr<FOnlineSessionSteam>(Subsystem->GetSessionInterface());
	FNamedOnlineSession* Session = SessionInt->GetNamedSession(SessionName);
	if (Session)
	{
		bool bUsesPresence = Session->SessionSettings.bUsesPresence;
		if (bUsesPresence != NewSessionSettings.bUsesPresence)
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't change presence settings on existing session %s, ignoring."), *SessionName.ToString());
		}

		Session->SessionSettings = NewSessionSettings;
		Session->SessionSettings.bUsesPresence = bUsesPresence;

#if WITH_ENGINE
		if (bUpdateOnlineData)
		{
			UWorld* World = GetWorldForOnline(Subsystem->GetInstanceName());

			// Server list update
			UpdatePublishedSettings(World, Session);
		}
#endif //WITH_ENGINE

		bWasSuccessful = true;
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("No session %s found to update with Steam backend"), *SessionName.ToString());
	}

	bIsComplete = true;
}

/**
 *	Async task is given a chance to trigger it's delegates
 */
void FOnlineAsyncTaskSteamUpdateServer::TriggerDelegates() 
{
	IOnlineSessionPtr SessionInt = Subsystem->GetSessionInterface();
	if (SessionInt.IsValid())
	{
		SessionInt->TriggerOnUpdateSessionCompleteDelegates(SessionName, bWasSuccessful);
	}
}

/**
 *	Get a human readable description of task
 */
FString FOnlineAsyncTaskSteamLogoffServer::ToString() const 
{
	return FString::Printf(TEXT("FOnlineAsyncTaskSteamLogoffServer bWasSuccessful: %d"), WasSuccessful());
}

/**
 * Give the async task time to do its work
 * Can only be called on the async task manager thread
 */
void FOnlineAsyncTaskSteamLogoffServer::Tick() 
{
	if (!bInit)
	{
		// @TODO ONLINE Listen Servers need to unset rich presence
		//SteamFriends()->SetRichPresence("connect", ""); for listed sessions
		SteamGameServer()->SetAdvertiseServerActive(false);  // renamed EnableHeartbeats to SetAdvertiseServerActive in 1.53
		SteamGameServer()->LogOff();
		bInit = true;
	}

	// Wait for the disconnect
	FOnlineSessionSteamPtr SessionInt = StaticCastSharedPtr<FOnlineSessionSteam>(Subsystem->GetSessionInterface());
	if (!SessionInt->bSteamworksGameServerConnected && !SessionInt->GameServerSteamId.IsValid())
	{
		bIsComplete = true;
		bWasSuccessful = true;
	}
	else
	{
		// Fallback timeout in case we don't hear from Steam
		float AsyncTimeout = ASYNC_TASK_TIMEOUT;
		GConfig->GetFloat(TEXT("OnlineSubsystemSteam"), TEXT("OnlineAsyncTaskSteamLogoffServerTimeout"), AsyncTimeout, GEngineIni);
		
		if (GetElapsedTime() >= AsyncTimeout)
		{
			SessionInt->bSteamworksGameServerConnected = false;
			SessionInt->GameServerSteamId = NULL;
			bIsComplete = true;
			bWasSuccessful = false;
		}
	}
}

/** 
 * Fills in the proxy search result with all the rules returned by the aux query
 *
 * @return true if session was successfully created, false otherwise
 */
bool FPendingSearchResultSteam::FillSessionFromServerRules()
{
	bool bSuccess = true;

	// Create the session info
	TSharedPtr<FOnlineSessionInfoSteam> SessionInfo = MakeShareable(new FOnlineSessionInfoSteam(ESteamSession::AdvertisedSessionClient, *ServerId));
	TSharedRef<FInternetAddrSteam> SteamP2PAddr = MakeShareable(new FInternetAddrSteam);

	FOnlineSession* Session = &PendingSearchResult.Session;

	// Make sure we hit the important keys
	int32 KeysFound = 0;
	int32 SteamAddrKeysFound = 0;

	for (FSteamSessionKeyValuePairs::TConstIterator It(ServerRules); It; ++It)
	{
// 		if (FCStringAnsi::Stricmp(TCHAR_TO_ANSI(*It.Key()), STEAMKEY_NUMPUBLICCONNECTIONS) == 0)
// 		{
// 			Session->SessionSettings.NumPublicConnections = FCString::Atoi(*It.Value());
// 			KeysFound++;
// 		}
// 		else if (FCStringAnsi::Stricmp(TCHAR_TO_ANSI(*It.Key()), STEAMKEY_NUMPRIVATECONNECTIONS) == 0)
// 		{
// 			Session->SessionSettings.NumPrivateConnections = FCString::Atoi(*It.Value());
// 			KeysFound++;
// 		}
		if (FCStringAnsi::Stricmp(TCHAR_TO_ANSI(*It.Key()), STEAMKEY_SESSIONFLAGS) == 0)
		{
			int32 BitShift = 0;
			int32 SessionFlags = FCString::Atoi(*It.Value());
			Session->SessionSettings.bShouldAdvertise = (SessionFlags & (1 << BitShift++)) ? true : false;
			Session->SessionSettings.bAllowJoinInProgress = (SessionFlags & (1 << BitShift++)) ? true : false;
			Session->SessionSettings.bIsLANMatch = (SessionFlags & (1 << BitShift++)) ? true : false;
			Session->SessionSettings.bIsDedicated = (SessionFlags & (1 << BitShift++)) ? true : false;
			Session->SessionSettings.bUsesStats = (SessionFlags & (1 << BitShift++)) ? true : false;
			Session->SessionSettings.bAllowInvites = (SessionFlags & (1 << BitShift++)) ? true : false;
			Session->SessionSettings.bUsesPresence = (SessionFlags & (1 << BitShift++)) ? true : false;
			Session->SessionSettings.bAllowJoinViaPresence = (SessionFlags & (1 << BitShift++)) ? true : false;
			Session->SessionSettings.bAllowJoinViaPresenceFriendsOnly = (SessionFlags & (1 << BitShift++)) ? true : false;
			Session->SessionSettings.bAntiCheatProtected = (SessionFlags & (1 << BitShift++)) ? true : false;
			KeysFound++;
		}
		else if (FCStringAnsi::Stricmp(TCHAR_TO_ANSI(*It.Key()), STEAMKEY_OWNINGUSERID) == 0)
		{
			uint64 UniqueId = FCString::Atoi64(*It.Value());
			if (UniqueId != 0)
			{
				Session->OwningUserId = FUniqueNetIdSteam::Create(UniqueId);
				KeysFound++;
			}
		}
		else if (FCStringAnsi::Stricmp(TCHAR_TO_ANSI(*It.Key()), STEAMKEY_OWNINGUSERNAME) == 0)
		{
			if (FCString::Strlen(*It.Value()) > 0)
			{
				Session->OwningUserName = It.Value();
				KeysFound++;
			}
		}
		else if (FCStringAnsi::Stricmp(TCHAR_TO_ANSI(*It.Key()), STEAMKEY_CONNECTIONMETHOD) == 0)
		{
			SessionInfo->ConnectionMethod = ToConnectionMethod(It.Value());
			++KeysFound;
		}
// 		else if (FCStringAnsi::Stricmp(TCHAR_TO_ANSI(*It.Key()), STEAMKEY_NUMOPENPRIVATECONNECTIONS) == 0)
// 		{
// 			Session->NumOpenPrivateConnections = FCString::Atoi(*It.Value());
// 			KeysFound++;
// 		}
// 		else if (FCStringAnsi::Stricmp(TCHAR_TO_ANSI(*It.Key()), STEAMKEY_NUMOPENPUBLICCONNECTIONS) == 0)
// 		{
// 			Session->NumOpenPublicConnections = FCString::Atoi(*It.Value());
// 			KeysFound++;
// 		}
		else if (FCStringAnsi::Stricmp(TCHAR_TO_ANSI(*It.Key()), STEAMKEY_P2PADDR) == 0)
		{
			FString KeyValue = It.Value();
			// Remove any protocol flags from the start.
			KeyValue.RemoveFromStart(STEAM_URL_PREFIX);

			uint64 SteamAddr = FCString::Atoi64(*KeyValue);
			if (SteamAddr != 0)
			{
				SteamP2PAddr->SetSteamID(CSteamID(SteamAddr));
				SteamAddrKeysFound++;
			}
		}
		else if (FCStringAnsi::Stricmp(TCHAR_TO_ANSI(*It.Key()), STEAMKEY_P2PPORT) == 0)
		{
			int32 Port = FCString::Atoi(*It.Value());
			SteamP2PAddr->SetPort(Port);
			SteamAddrKeysFound++;
		}
		else
		{
			FName NewKey;
			FOnlineSessionSetting NewSetting;
			if (SteamKeyToSessionSetting(*It.Key(), TCHAR_TO_ANSI(*It.Value()), NewKey, NewSetting))
			{
				Session->SessionSettings.Set(NewKey, NewSetting);
			}
			else
			{
				bSuccess = false;
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to parse setting from key %s value %s"), *It.Key(), *It.Value());
			}
		}
	}

	// Verify success with all required keys found
	// If the user has SteamNetworking off, then we should just check if their host addressing is correct.
	if (bSuccess && (KeysFound == STEAMKEY_NUMREQUIREDSERVERKEYS) && (SteamAddrKeysFound == 2 || (HostAddr.IsValid() && HostAddr->IsValid())))
	{
		SessionInfo->HostAddr = HostAddr;
		SessionInfo->SteamP2PAddr = SteamP2PAddr;

		Session->SessionInfo = SessionInfo;
		return true;
	}

	return false;
}

/**
 * Got data on a rule on the server -- you'll get one of these per rule defined on
 * the server you are querying
 */ 
void FPendingSearchResultSteam::RulesResponded(const char *pchRule, const char *pchValue)
{
	UE_LOG_ONLINE_SESSION(Warning, TEXT("Rules response %s %s"), UTF8_TO_TCHAR(pchRule), UTF8_TO_TCHAR(pchValue));
	ParentQuery->ElapsedTime = 0.0f;
	ServerRules.Add(UTF8_TO_TCHAR(pchRule), UTF8_TO_TCHAR(pchValue));
}

/**
 * The server failed to respond to the request for rule details
 */ 
void FPendingSearchResultSteam::RulesFailedToRespond()
{
	UE_LOG_ONLINE_SESSION(Warning, TEXT("Rules failed to respond for server"));
	ParentQuery->ElapsedTime = 0.0f;
	RemoveSelf();
}

/**
 *  The server has finished responding to the rule details request 
 * (ie, you won't get anymore RulesResponded callbacks)
 */
void FPendingSearchResultSteam::RulesRefreshComplete()
{
	UE_LOG_ONLINE_SESSION(Warning, TEXT("Rules refresh complete"));
	ParentQuery->ElapsedTime = 0.0f;

	// Only append this data if there is an existing search (NULL CurrentSessionSearch implies no active search query)
	FOnlineSessionSteamPtr SessionInt = StaticCastSharedPtr<FOnlineSessionSteam>(ParentQuery->Subsystem->GetSessionInterface());
	if (SessionInt.IsValid() && SessionInt->CurrentSessionSearch.IsValid() && SessionInt->CurrentSessionSearch->SearchState == EOnlineAsyncTaskState::InProgress)
	{
		if (FillSessionFromServerRules())
		{
			// Transfer rules to actual search results
			FOnlineSessionSearchResult* SearchResult = new (ParentQuery->SearchSettings->SearchResults) FOnlineSessionSearchResult(PendingSearchResult);
			SearchResult->Session.SessionInfo = PendingSearchResult.Session.SessionInfo;
			if (!SearchResult->IsValid())
			{
				// Remove the failed element
				ParentQuery->SearchSettings->SearchResults.RemoveAtSwap(ParentQuery->SearchSettings->SearchResults.Num() - 1);
			}
		}
	}

	RemoveSelf();
}

/**
 * Remove this search result from the parent's list of pending entries
 */
void FPendingSearchResultSteam::RemoveSelf()
{	
	for (int32 SearchIdx=0; SearchIdx< ParentQuery->PendingSearchResults.Num(); SearchIdx++)
	{
		if (ParentQuery->PendingSearchResults[SearchIdx].ServerId == ServerId)
		{
			ParentQuery->PendingSearchResults.RemoveAtSwap(SearchIdx);
			break;
		}
	}
}

/**
 * Cancel this rules request
 */
void FPendingSearchResultSteam::CancelQuery()
{
	if (ServerQueryHandle != HSERVERQUERY_INVALID)
	{
		SteamMatchmakingServers()->CancelServerQuery(ServerQueryHandle);
		ServerQueryHandle = HSERVERQUERY_INVALID;
	}
}


#ifdef _MSC_VER
#pragma warning(push)
#pragma warning(disable : 6385) // Disable dubious Static Analysis warning C6385 on windows. See MS TechNote Here https://developercommunity2.visualstudio.com/t/C6385-False-Positive/878703
#endif

/**
 *  Create the proper query for the server list based on the given search settings
 *
 * @param OutFilter Steam structure containing the proper filters
 * @param NumFilters number of filters contained in the array above
 */
void FOnlineAsyncTaskSteamFindServerBase::CreateQuery(MatchMakingKeyValuePair_t** OutFilter, int32& NumFilters)
{
	// Copy the params so we can remove the values as we use them
	FOnlineSearchSettings TempSearchSettings = SearchSettings->QuerySettings;

    // Include enough space for all search parameters plus the required one "gamedir" below
	int32 MaxFilters = TempSearchSettings.SearchParams.Num() + 1;

	*OutFilter = new MatchMakingKeyValuePair_t[MaxFilters];
	MatchMakingKeyValuePair_t* Filters = *OutFilter; 

	int32 KeySize = sizeof(Filters[0].m_szKey);
	int32 ValueSize = sizeof(Filters[0].m_szValue);

    NumFilters = 0;
	// Filter must match at least our game
	FCStringAnsi::Strncpy(Filters[NumFilters].m_szKey, "gamedir", KeySize);
	FCStringAnsi::Strncpy(Filters[NumFilters].m_szValue, UE_PROJECT_STEAMGAMEDIR, ValueSize);
	NumFilters++;

	FString MapName;
	if (TempSearchSettings.Get(SETTING_MAPNAME, MapName) && !MapName.IsEmpty())
	{
		// Server passes the filter if the server is playing the specified map.
		FCStringAnsi::Strncpy(Filters[NumFilters].m_szKey, "map", KeySize);
		FCStringAnsi::Strncpy(Filters[NumFilters].m_szValue, TCHAR_TO_ANSI(*MapName), ValueSize);
		NumFilters++;
	}
	TempSearchSettings.SearchParams.Remove(SETTING_MAPNAME);

	FString HostIp;
	if (TempSearchSettings.Get(SEARCH_STEAM_HOSTIP, HostIp) && !HostIp.IsEmpty())
	{
		// Server passes the filter if it passed a valid host ip.
		FCStringAnsi::Strncpy(Filters[NumFilters].m_szKey, "gameaddr", KeySize);
		FCStringAnsi::Strncpy(Filters[NumFilters].m_szValue, TCHAR_TO_ANSI(*HostIp), ValueSize);
		NumFilters++;
	}
	TempSearchSettings.SearchParams.Remove(SEARCH_STEAM_HOSTIP);

	int32 DedicatedOnly = 0;
	if (TempSearchSettings.Get(SEARCH_DEDICATED_ONLY, DedicatedOnly) && DedicatedOnly != 0)
	{
		// Server passes the filter if it passed true to SetDedicatedServer.
		FCStringAnsi::Strncpy(Filters[NumFilters].m_szKey, "dedicated", KeySize);
		FCStringAnsi::Strncpy(Filters[NumFilters].m_szValue, "true", ValueSize);
		NumFilters++;
	}
	TempSearchSettings.SearchParams.Remove(SEARCH_DEDICATED_ONLY);

	int32 SecureOnly = 0;
	if (TempSearchSettings.Get(SEARCH_SECURE_SERVERS_ONLY, SecureOnly) && SecureOnly != 0)
	{
		// Server passes the filter if the server is VAC-enabled.
		FCStringAnsi::Strncpy(Filters[NumFilters].m_szKey, "secure", KeySize);
		FCStringAnsi::Strncpy(Filters[NumFilters].m_szValue, "true", ValueSize);
		NumFilters++;
	}
	TempSearchSettings.SearchParams.Remove(SEARCH_SECURE_SERVERS_ONLY);

	int32 EmptyOnly = 0;
	if (TempSearchSettings.Get(SEARCH_EMPTY_SERVERS_ONLY, EmptyOnly) && EmptyOnly != 0)
	{
		// Server passes the filter if it doesn't have any players.
		FCStringAnsi::Strncpy(Filters[NumFilters].m_szKey, "noplayers", KeySize);
		FCStringAnsi::Strncpy(Filters[NumFilters].m_szValue, "true", ValueSize);
		NumFilters++;
	}
	TempSearchSettings.SearchParams.Remove(SEARCH_EMPTY_SERVERS_ONLY);


	// TEMP!!!!
	return;
#if 0
	/**
	 * "full"		- not full
	 * "empty"		- not empty
	 * "proxy"		- a relay server
	 */
	if (NumFilters <= MaxFilters)
	{
		/** Filter out key value pairs */
		TArray<FString> Clauses;
		FString CurrentClause;
		for (FSearchParams::TConstIterator It(TempSearchSettings.SearchParams); It; ++It)
		{
			const FName Key = It.Key();
			const FOnlineSessionSearchParam& SearchParam = It.Value();

			FString KeyStr;
			if (SessionKeyToSteamKey(Key, SearchParam.Data, KeyStr))
			{
				if (SearchParam.ComparisonOp == EOnlineComparisonOp::Equals)
				{
					FString NewParam = FString::Printf(TEXT("%s:%s"), *KeyStr, *SearchParam.Data.ToString());
					if (NewParam.Len() <= ValueSize)
					{
						if (NewParam.Len() + CurrentClause.Len() < ValueSize)
						{
							if (CurrentClause.IsEmpty())
							{
								CurrentClause = NewParam;
							}
							else
							{
								// Continue to add to the clause
								CurrentClause = CurrentClause + "," + NewParam;
							}
						}
						else
						{
							// Create a new clause
							Clauses.Add(CurrentClause);
							CurrentClause = NewParam;
						}
					}
					else
					{
						UE_LOG_ONLINE_SESSION(Warning, TEXT("Skipping search clause due to size: %s"), *NewParam);
					}
				}
			}
		}

		// Add the remainder clause
		if (!CurrentClause.IsEmpty())
		{
			Clauses.Add(CurrentClause);
		}

		if (Clauses.Num() > 0)
		{
			// Make sure there is room (Clauses + "and" clause if more than one)
			if (NumFilters + Clauses.Num() + (Clauses.Num() > 1 ? 1 : 0) <= MaxFilters)
			{
				if (Clauses.Num() > 1)
				{
					// "and" (x1 && x2 && ... && xn) where n is number of clauses
					FCStringAnsi::Strncpy(Filters[NumFilters].m_szKey, "and", KeySize);
					FCStringAnsi::Strncpy(Filters[NumFilters].m_szValue, TCHAR_TO_UTF8(*FString::FromInt(Clauses.Num())), ValueSize);
					NumFilters++;

					for (int32 ClauseIdx=0; ClauseIdx < Clauses.Num(); ClauseIdx++)
					{
						// Server passes the filter if the server's game data (ISteamGameServer::SetGameData) contains all of the
						// specified strings. The value field is a comma-delimited list of strings to match.
						FCStringAnsi::Strncpy(Filters[NumFilters].m_szKey, "gamedataand", KeySize);
						FCStringAnsi::Strncpy(Filters[NumFilters].m_szValue, TCHAR_TO_UTF8(*Clauses[ClauseIdx]), ValueSize);
						NumFilters++;
					}
				}
				else
				{
					// Server passes the filter if the server's game data (ISteamGameServer::SetGameData) contains all of the
					// specified strings. The value field is a comma-delimited list of strings to match.
					FCStringAnsi::Strncpy(Filters[NumFilters].m_szKey, "gamedataand", KeySize);
					FCStringAnsi::Strncpy(Filters[NumFilters].m_szValue, TCHAR_TO_UTF8(*Clauses[0]), ValueSize);
					NumFilters++;
				}
			}
		}
	}
#endif
}

#ifdef _MSC_VER
#pragma warning(pop)
#endif

/**
 *	Create a search result from a server response
 *
 * @param ServerDetails Steam server details
 */
void FOnlineAsyncTaskSteamFindServerBase::ParseSearchResult(class gameserveritem_t* ServerDetails)
{
#if WITH_ENGINE
	TSharedRef<FInternetAddr> ServerAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();

	ServerAddr->SetIp(ServerDetails->m_NetAdr.GetIP());
	ServerAddr->SetPort(ServerDetails->m_NetAdr.GetConnectionPort());
	int32 ServerQueryPort = ServerDetails->m_NetAdr.GetQueryPort();

	UE_LOG_ONLINE_SESSION(Warning, TEXT("Server response IP:%s"), *ServerAddr->ToString(false));
	if (ServerDetails->m_bHadSuccessfulResponse)
	{
		FString GameTags(UTF8_TO_TCHAR(ServerDetails->m_szGameTags));

		// Check for build compatibility
		int32 ServerBuildId = 0;
		int32 BuildUniqueId = GetBuildUniqueId();

		TArray<FString> TagArray;
		GameTags.ParseIntoArray(TagArray, TEXT(","), true);
		if (TagArray.Num() > 0 && TagArray[0].StartsWith(STEAMKEY_BUILDUNIQUEID))
		{
			ServerBuildId = FCString::Atoi(*TagArray[0].Mid(UE_ARRAY_COUNT(STEAMKEY_BUILDUNIQUEID)));
		}

		if (ServerBuildId == BuildUniqueId)
		{
			// Create a new pending search result 
			FPendingSearchResultSteam* NewPendingSearch = new FPendingSearchResultSteam(this);
			PendingSearchResults.Add(NewPendingSearch);
			NewPendingSearch->ServerId = FUniqueNetIdSteam::Create(ServerDetails->m_steamID);
			NewPendingSearch->HostAddr = ServerAddr;

			// Fill search result members
			FOnlineSessionSearchResult* NewSearchResult = &NewPendingSearch->PendingSearchResult;
			NewSearchResult->PingInMs = FMath::Clamp(ServerDetails->m_nPing, 0, MAX_QUERY_PING);

			// Fill session members
			FOnlineSession* NewSession = &NewSearchResult->Session;

			//NewSession->OwningUserId = ;
			NewSession->OwningUserName = UTF8_TO_TCHAR(ServerDetails->GetName());

			NewSession->NumOpenPublicConnections = ServerDetails->m_nMaxPlayers - ServerDetails->m_nPlayers;
			NewSession->NumOpenPrivateConnections = 0;

			// Fill session settings members
			NewSession->SessionSettings.NumPublicConnections = ServerDetails->m_nMaxPlayers;
			NewSession->SessionSettings.NumPrivateConnections = 0;
			NewSession->SessionSettings.bAntiCheatProtected = ServerDetails->m_bSecure ? true : false;
			NewSession->SessionSettings.Set(SETTING_MAPNAME, FString(UTF8_TO_TCHAR(ServerDetails->m_szMap)), EOnlineDataAdvertisementType::ViaOnlineService);

			// Start a rules request for this new result
			NewPendingSearch->ServerQueryHandle = SteamMatchmakingServersPtr->ServerRules(ServerDetails->m_NetAdr.GetIP(), ServerQueryPort, NewPendingSearch);
			if (NewPendingSearch->ServerQueryHandle == HSERVERQUERY_INVALID)
			{
				// Remove the failed element
				PendingSearchResults.RemoveAtSwap(PendingSearchResults.Num() - 1);
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Removed incompatible build: ServerBuildUniqueId = 0x%08x, GetBuildUniqueId() = 0x%08x"),
				ServerBuildId, BuildUniqueId);
		}
	}
#endif //WITH_ENGINE
}

/**
 * Give the async task time to do its work
 * Can only be called on the async task manager thread
 */
void FOnlineAsyncTaskSteamFindServerBase::Tick()
{
	ISteamUtils* SteamUtilsPtr = SteamUtils();
	check(SteamUtilsPtr);

	if (!bInit)
	{
		SteamMatchmakingServersPtr = SteamMatchmakingServers();
		check(SteamMatchmakingServersPtr);

		int32 NumFilters = 0;
		MatchMakingKeyValuePair_t* Filters = NULL;
		CreateQuery(&Filters, NumFilters);

#if DEBUG_STEAM_FILTERS
		for (int32 FilterIdx=0; FilterIdx<NumFilters; FilterIdx++)
		{
			UE_LOG_ONLINE_SESSION(Verbose, TEXT(" \"%s\" \"%s\" "), UTF8_TO_TCHAR(Filters[FilterIdx].m_szKey), UTF8_TO_TCHAR(Filters[FilterIdx].m_szValue));
		}
#endif

		if (SearchSettings->MaxSearchResults <= 0)
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineAsyncTaskSteamFindServerBase::Tick - SearchSettings->MaxSearchResults should be greater than 0, but it is currently %d. No search results will be found."), SearchSettings->MaxSearchResults);
		}

		if (SearchSettings->bIsLanQuery)
		{
			ServerListRequestHandle = SteamMatchmakingServersPtr->RequestLANServerList(Subsystem->GetSteamAppId(), this);
		}
		else
		{
			ServerListRequestHandle = SteamMatchmakingServersPtr->RequestInternetServerList(Subsystem->GetSteamAppId(), &Filters, NumFilters, this);
		}

		if (ServerListRequestHandle == NULL)
		{
			// Invalid API call
			bIsComplete = true;
			bWasSuccessful = false;
		}

		// Preallocate space for results
		PendingSearchResults.Empty(SearchSettings->MaxSearchResults);

		delete [] Filters;
		bInit = true;
	}

	ElapsedTime += 1.0f/16.0f;

	// Cancel query when we've reached our requested limit
	bool bReachedSearchLimit = (SearchSettings->SearchResults.Num() >= SearchSettings->MaxSearchResults) ? true : false;
	// Check for activity timeout
	float AsyncTimeout = ASYNC_TASK_TIMEOUT;
	GConfig->GetFloat(TEXT("OnlineSubsystemSteam"), TEXT("OnlineAsyncTaskSteamFindServerBaseTimeout"), AsyncTimeout, GEngineIni);
	
	bool bTimedOut = (ElapsedTime >= AsyncTimeout) ? true : false;
	// Check for proper completion
	bool bServerSearchComplete = (bServerRefreshComplete && PendingSearchResults.Num() == 0) ? true : false;
	if ( bReachedSearchLimit || bTimedOut || bServerSearchComplete)
	{
		bIsComplete = true;
		bWasSuccessful = true;
	}

	if (bIsComplete)
	{
		// Cancel further server queries (may trigger RefreshComplete delegate)
		if (ServerListRequestHandle != NULL)
		{
			SteamMatchmakingServersPtr->CancelQuery(ServerListRequestHandle);
			SteamMatchmakingServersPtr->ReleaseRequest(ServerListRequestHandle);
			ServerListRequestHandle = NULL;
		}

		// Cancel further rules queries
		for (int32 PendingIdx=0; PendingIdx<PendingSearchResults.Num(); ++PendingIdx)
		{
			PendingSearchResults[PendingIdx].CancelQuery();
		}
		PendingSearchResults.Empty();
	}
}

/**
 * Called by the SteamAPI when a server has successfully responded
 */
void FOnlineAsyncTaskSteamFindServerBase::ServerResponded(HServerListRequest Request, int iServer)
{
	ElapsedTime = 0.0f;

	gameserveritem_t* Server = SteamMatchmakingServersPtr->GetServerDetails(Request, iServer);
	if (Server != NULL)
	{
		// Filter out servers that don't match our appid here
		if (!Server->m_bDoNotRefresh && Server->m_nAppID == SteamUtils()->GetAppID())
		{
			ParseSearchResult(Server);
		}
	}
}

/**
 * Called by the SteamAPI when a server has failed to respond
 */
void FOnlineAsyncTaskSteamFindServerBase::ServerFailedToRespond(HServerListRequest Request, int iServer)
{
#if WITH_ENGINE
	ElapsedTime = 0.0f;

	gameserveritem_t* Server = SteamMatchmakingServersPtr->GetServerDetails(Request, iServer);
	if (Server != NULL)
	{
		TSharedRef<FInternetAddr> ServerAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();
		int32 ServerQueryPort = 0;

		ServerAddr->SetIp(Server->m_NetAdr.GetIP());
		ServerAddr->SetPort(Server->m_NetAdr.GetConnectionPort());
		ServerQueryPort = Server->m_NetAdr.GetQueryPort();

		UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to respond IP:%s"), *ServerAddr->ToString(false));

		// Filter out servers that don't match our appid here
		if (Server->m_nAppID == SteamUtils()->GetAppID())
		{
		}
	}
#endif //WITH_ENGINE
}

/**
 * Called by the SteamAPI when all server requests for the list have completed
 */
void FOnlineAsyncTaskSteamFindServerBase::RefreshComplete(HServerListRequest Request, EMatchMakingServerResponse Response)
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("Server query complete %s"), *SteamMatchMakingServerResponseString(Response));
	bServerRefreshComplete = true;
	ElapsedTime = 0.0f;
}

/**
 * Give the async task a chance to marshal its data back to the game thread
 * Can only be called on the game thread by the async task manager
 */
void FOnlineAsyncTaskSteamFindServerBase::Finalize()
{
	FOnlineSessionSteamPtr SessionInt = StaticCastSharedPtr<FOnlineSessionSteam>(Subsystem->GetSessionInterface());

	SearchSettings->SearchState = bWasSuccessful ? EOnlineAsyncTaskState::Done : EOnlineAsyncTaskState::Failed;
	if (bWasSuccessful)
	{
		if (SearchSettings->SearchResults.Num() > 0)
		{
			// Allow game code to sort the servers
			SearchSettings->SortSearchResults();
		}
	}

	if (SessionInt->CurrentSessionSearch.IsValid() && SearchSettings == SessionInt->CurrentSessionSearch)
	{
		SessionInt->CurrentSessionSearch = NULL;
	}
}

/**
*	Get a human readable description of task
*/
FString FOnlineAsyncTaskSteamFindServerForInviteSession::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncTaskSteamFindServerForInvite bWasSuccessful: %d Results: %d"), WasSuccessful(), SearchSettings->SearchResults.Num());
}


/**
*	Async task is given a chance to trigger it's delegates
*/
void FOnlineAsyncTaskSteamFindServerForInviteSession::TriggerDelegates()
{
	if (FindServerInviteCompleteWithUserIdDelegates.IsBound() && LocalUserNum >= 0)
	{
		if (bWasSuccessful && SearchSettings->SearchResults.Num() > 0)
		{
			FindServerInviteCompleteWithUserIdDelegates.Broadcast(bWasSuccessful, LocalUserNum, FUniqueNetIdSteam::Create(SteamUser()->GetSteamID()), SearchSettings->SearchResults[0]);
		}
		else
		{
			FOnlineSessionSearchResult EmptyResult;
			FindServerInviteCompleteWithUserIdDelegates.Broadcast(bWasSuccessful, LocalUserNum, FUniqueNetIdSteam::Create(SteamUser()->GetSteamID()), EmptyResult);
		}
	}
}

/**
*	Get a human readable description of task
*/
FString FOnlineAsyncTaskSteamFindServerForFriendSession::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncTaskSteamFindServerForFriend bWasSuccessful: %d Results: %d"), WasSuccessful(), SearchSettings->SearchResults.Num());
}

/**
*	Async task is given a chance to trigger it's delegates
*/
void FOnlineAsyncTaskSteamFindServerForFriendSession::TriggerDelegates()
{
	if (FindServerInviteCompleteDelegates.IsBound() && LocalUserNum >= 0)
	{
		if (bWasSuccessful && SearchSettings->SearchResults.Num() > 0)
		{
			FindServerInviteCompleteDelegates.Broadcast(LocalUserNum, bWasSuccessful, SearchSettings->SearchResults);
		}
		else
		{
			TArray<FOnlineSessionSearchResult> EmptyResult;
			FindServerInviteCompleteDelegates.Broadcast(LocalUserNum, bWasSuccessful, EmptyResult);
		}
	}
}

/**
 *	Get a human readable description of task
 */
FString FOnlineAsyncTaskSteamFindServers::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncTaskSteamFindServers bWasSuccessful: %d Results: %d"), WasSuccessful(), SearchSettings->SearchResults.Num());
}

/**
 *	Async task is given a chance to trigger it's delegates
 */
void FOnlineAsyncTaskSteamFindServers::TriggerDelegates()
{
	if (FindServersCompleteDelegates.IsBound())
	{
		FindServersCompleteDelegates.Broadcast(bWasSuccessful);
	}
}

/**
 * Give the async task a chance to marshal its data back to the game thread
 * Can only be called on the game thread by the async task manager
 */
void FOnlineAsyncEventSteamInviteAccepted::Finalize()
{
	FOnlineSessionSteamPtr SessionInt = StaticCastSharedPtr<FOnlineSessionSteam>(Subsystem->GetSessionInterface());
	if (SessionInt.IsValid() && !SessionInt->CurrentSessionSearch.IsValid())
	{
		// Create a search settings object
		TSharedRef<FOnlineSessionSearch> SearchSettings = MakeShareable(new FOnlineSessionSearch());
		SessionInt->CurrentSessionSearch = SearchSettings;
		SessionInt->CurrentSessionSearch->SearchState = EOnlineAsyncTaskState::InProgress;

		TCHAR ParsedURL[1024];
		if (!FParse::Value(*ConnectionURL, TEXT("SteamConnectIP="), ParsedURL, UE_ARRAY_COUNT(ParsedURL)))
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineAsyncEventSteamInviteAccepted: Failed to parse connection URL"));
			return;
		}

		// Determine the port
		int32 Port = 0;
		TCHAR* PortToken = FCString::Strchr(ParsedURL, ':');
		if (PortToken)
		{
			Port = FCString::Atoi(PortToken+1);
			PortToken[0] = '\0';
		}

		Port = (Port > 0) ? Port : Subsystem->GetGameServerGamePort();

		// Parse the address
#if WITH_ENGINE
		TSharedPtr<FInternetAddr> IpAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetAddressFromString(ParsedURL);
		if (IpAddr.IsValid())
		{
			SessionInt->CurrentSessionSearch->QuerySettings.Set(FName(SEARCH_STEAM_HOSTIP), IpAddr->ToString(false), EOnlineComparisonOp::Equals);
			FOnlineAsyncTaskSteamFindServerForInviteSession* NewTask = new FOnlineAsyncTaskSteamFindServerForInviteSession(Subsystem, SearchSettings, LocalUserNum, SessionInt->OnSessionUserInviteAcceptedDelegates);
			Subsystem->QueueAsyncTask(NewTask);
		}
#else //WITH_ENGINE
		ensure(false); //Launch invites are not expected to work without engine
#endif //WITH_ENGINE
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Invalid session or search already in progress when accepting invite.  Ignoring invite request."));
	}
}

