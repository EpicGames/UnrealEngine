// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSessionInterfaceSteam.h"
#include "OnlineAsyncTaskManagerSteam.h"

class FInternetAddr;
class FOnlineSubsystemSteam;

/** Well defined keys for use with Steam game servers */
#define SEARCH_STEAM_HOSTIP TEXT("SteamHostIp")

/** 
 *  Async task for creating a Steam advertised server
 */
class FOnlineAsyncTaskSteamCreateServer : public FOnlineAsyncTaskSteam
{
private:

	/** Has this request been started */
	bool bInit;
	/** Name of session being created */
	FName SessionName;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamCreateServer() = delete;

public:

	FOnlineAsyncTaskSteamCreateServer(class FOnlineSubsystemSteam* InSubsystem, FName InSessionName) :
		FOnlineAsyncTaskSteam(InSubsystem, k_uAPICallInvalid),
		bInit(false),
		SessionName(InSessionName)
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override;

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override;

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override;
	
	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override;
};

/** 
 *  Async task to update a single Steam lobby
 */
class FOnlineAsyncTaskSteamUpdateServer : public FOnlineAsyncTaskSteam
{
private:

	/** Name of session being created */
	FName SessionName;
	/** New session settings to apply */
	FOnlineSessionSettings NewSessionSettings;
	/** Should the online platform refresh as well */
	bool bUpdateOnlineData;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamUpdateServer() : 
		SessionName(NAME_None),
		bUpdateOnlineData(false)
	{
	}

public:

	/** Constructor */
	FOnlineAsyncTaskSteamUpdateServer(class FOnlineSubsystemSteam* InSubsystem, FName InSessionName, bool bInUpdateOnlineData, const FOnlineSessionSettings& InNewSessionSettings) :
		FOnlineAsyncTaskSteam(InSubsystem, k_uAPICallInvalid),
		SessionName(InSessionName),
		NewSessionSettings(InNewSessionSettings),
		bUpdateOnlineData(bInUpdateOnlineData)
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override;

	/**
	* Give the async task time to do its work
	* Can only be called on the async task manager thread
	*/
	virtual void Tick() override;
	
	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override;
};

/** 
 *  Async task for shutting down an advertised game erver
 */
class FOnlineAsyncTaskSteamLogoffServer : public FOnlineAsyncTaskSteam
{
private:

	/** Has this request been started */
	bool bInit;
	/** Name of session */
	FName SessionName;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamLogoffServer() : 
		bInit(false),
		SessionName(NAME_None)
	{
	}

public:

	/** Constructor */
	FOnlineAsyncTaskSteamLogoffServer(class FOnlineSubsystemSteam* InSubsystem, FName InSessionName) :
		FOnlineAsyncTaskSteam(InSubsystem, k_uAPICallInvalid),
		bInit(false),
		SessionName(InSessionName)
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override;

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override;
};

/**
 * Container for a single search result returned by the initial server query
 * currently waiting for the Steam rules to be returned before creating a final search 
 * result to associate with the currently running query
 */
class FPendingSearchResultSteam final : public ISteamMatchmakingRulesResponse
{
	/** Hidden on purpose */
	FPendingSearchResultSteam() = delete;

PACKAGE_SCOPE:

	/** Reference to original search query */
	class FOnlineAsyncTaskSteamFindServerBase* ParentQuery;
	/** Handle to current rules response request with Steam */
	HServerQuery ServerQueryHandle;
    /** Steam Id of the server result */
	FUniqueNetIdSteamRef ServerId;
    /** Host address of the server result (PublicIP) */
	TSharedPtr<FInternetAddr> HostAddr;
	/** Placeholder for all returned rules until RulesRefreshComplete call */
	FSteamSessionKeyValuePairs ServerRules;
	/** Proxy search result until it is known to be valid */
	FOnlineSessionSearchResult PendingSearchResult;

    /** 
     * Fills in the proxy search result with all the rules returned by the aux query
     *
     * @return true if session was successfully created, false otherwise
     */
	bool FillSessionFromServerRules();

	/**
	 * Remove this search result from the parent's list of pending entries
	 */
	void RemoveSelf();

public:

    /** Constructor */
	FPendingSearchResultSteam(class FOnlineAsyncTaskSteamFindServerBase* InParentQuery):
		ParentQuery(InParentQuery),
		ServerQueryHandle(HSERVERQUERY_INVALID),
		ServerId(FUniqueNetIdSteam::EmptyId())
	{

	}

	~FPendingSearchResultSteam()
	{
		CancelQuery();
	}

	/**
	 * Got data on a rule on the server -- you'll get one of these per rule defined on
	 * the server you are querying
	 */ 
	virtual void RulesResponded(const char *pchRule, const char *pchValue);

	/**
	 * The server failed to respond to the request for rule details
	 */ 
	virtual void RulesFailedToRespond();

	/**
	 *  The server has finished responding to the rule details request 
	 * (ie, you won't get any more RulesResponded callbacks)
	 */
	virtual void RulesRefreshComplete();

	/**
	 * Cancel this rules request
	 */
	void CancelQuery();
};

/** 
 *  Base Async task for finding game servers advertised on the Steam backend (no delegates triggered)
 */
class FOnlineAsyncTaskSteamFindServerBase : public FOnlineAsyncTaskSteam, public ISteamMatchmakingServerListResponse
{
private:

	/** Has this request been started */
	bool bInit;
	/** Did the initial request complete */
	bool bServerRefreshComplete;
	/** Cached pointer to Steam interface */
	ISteamMatchmakingServers* SteamMatchmakingServersPtr;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamFindServerBase() : 
		bInit(false),
		bServerRefreshComplete(false),
		SteamMatchmakingServersPtr(NULL),
		ElapsedTime(0.0f),
		SearchSettings(NULL),
		ServerListRequestHandle(NULL)
	{
	}

PACKAGE_SCOPE:

	/** Timeout value for Steam bug */
	float ElapsedTime;
	/** Array of search results returned but waiting for rules response */
	TIndirectArray<FPendingSearchResultSteam> PendingSearchResults;
	/** Search settings specified for the query */
	TSharedPtr<class FOnlineSessionSearch> SearchSettings;
	/** Server list request handle */
	HServerListRequest ServerListRequestHandle;

public:

	FOnlineAsyncTaskSteamFindServerBase(class FOnlineSubsystemSteam* InSubsystem, const TSharedPtr<class FOnlineSessionSearch>& InSearchSettings) :
		FOnlineAsyncTaskSteam(InSubsystem, k_uAPICallInvalid),
		bInit(false),
		bServerRefreshComplete(false),
		SteamMatchmakingServersPtr(NULL),
		ElapsedTime(0.0f),
		SearchSettings(InSearchSettings),
		ServerListRequestHandle(NULL)
	{	
	}

	/**
     *  Create the proper query for the server list based on the given search settings
     *
     * @param OutFilter Steam structure containing the proper filters
     * @param NumFilters number of filters contained in the array above
     */
	virtual void CreateQuery(MatchMakingKeyValuePair_t** OutFilter, int32& NumFilters);

	/**
	 *	Create a proxy search result from a server response, triggers additional rules query
	 *
	 * @param ServerDetails Steam server details
	 */
	void ParseSearchResult(class gameserveritem_t* ServerDetails);

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override;

	/**
	 * Called by the SteamAPI when a server has successfully responded
	 */
	void ServerResponded(HServerListRequest Request, int iServer) override;

	/**
	 * Called by the SteamAPI when a server has failed to respond
	 */
	void ServerFailedToRespond(HServerListRequest Request, int iServer) override;

	/**
	 * Called by the SteamAPI when all server requests for the list have completed
	 */
	void RefreshComplete(HServerListRequest Request, EMatchMakingServerResponse Response) override;

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override;
};


DECLARE_MULTICAST_DELEGATE_FourParams(FOnAsyncFindServerInviteCompleteWithNetId, const bool, const int32, FUniqueNetIdPtr, const class FOnlineSessionSearchResult&);
typedef FOnAsyncFindServerInviteCompleteWithNetId::FDelegate FOnAsyncFindServerInviteCompleteWithNetIdDelegate;

class FOnlineAsyncTaskSteamFindServerForInviteSession : public FOnlineAsyncTaskSteamFindServerBase
{
protected:
	/** User initiating the request */
	int32 LocalUserNum;

	FOnAsyncFindServerInviteCompleteWithNetId FindServerInviteCompleteWithUserIdDelegates;

public:
	FOnlineAsyncTaskSteamFindServerForInviteSession(class FOnlineSubsystemSteam* InSubsystem, const TSharedPtr<class FOnlineSessionSearch>& InSearchSettings, int32 InLocalUserNum, FOnAsyncFindServerInviteCompleteWithNetId& InDelegates)
		: FOnlineAsyncTaskSteamFindServerBase(InSubsystem, InSearchSettings)
		, LocalUserNum(InLocalUserNum)
		, FindServerInviteCompleteWithUserIdDelegates(InDelegates)
	{
	}

	/**
	*	Get a human readable description of task
	*/
	virtual FString ToString() const override;

	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override;
};

class FOnlineAsyncTaskSteamFindServerForFriendSession : public FOnlineAsyncTaskSteamFindServerBase
{
protected:
	/** User initiating the request */
	int32 LocalUserNum;

	FOnFindFriendSessionComplete FindServerInviteCompleteDelegates;

public:
	FOnlineAsyncTaskSteamFindServerForFriendSession(class FOnlineSubsystemSteam* InSubsystem, const TSharedPtr<class FOnlineSessionSearch>& InSearchSettings, int32 InLocalUserNum, FOnFindFriendSessionComplete& InDelegates)
		: FOnlineAsyncTaskSteamFindServerBase(InSubsystem, InSearchSettings)
		, LocalUserNum(InLocalUserNum)
		, FindServerInviteCompleteDelegates(InDelegates)
	{
	}

	/**
	*	Get a human readable description of task
	*/
	virtual FString ToString() const override;

	/**
	*	Async task is given a chance to trigger it's delegates
	*/
	virtual void TriggerDelegates() override;
};

/**
 * Delegate fired when the search for servers has completed
 *
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnAsyncFindServersComplete, bool);
typedef FOnAsyncFindServersComplete::FDelegate FOnAsyncFindServersCompleteDelegate;

/**
 *	Async task for finding multiple servers and signaling the proper delegate on completion
 */
class FOnlineAsyncTaskSteamFindServers : public FOnlineAsyncTaskSteamFindServerBase
{
protected:

	/** General "find servers" delegate */
	FOnAsyncFindServersComplete FindServersCompleteDelegates;

public:

	FOnlineAsyncTaskSteamFindServers(class FOnlineSubsystemSteam* InSubsystem, const TSharedPtr<class FOnlineSessionSearch>& InSearchSettings, FOnAsyncFindServersComplete& InDelegates) :
		FOnlineAsyncTaskSteamFindServerBase(InSubsystem, InSearchSettings),
		FindServersCompleteDelegates(InDelegates)
	{	
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override;

	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override;
};

/**
 *	Turns a friends accepted invite request into a valid search result (server list version)
 */
class FOnlineAsyncEventSteamInviteAccepted : public FOnlineAsyncEvent<FOnlineSubsystemSteam>
{
	/** Friend who invited the user */
	FUniqueNetIdSteamRef FriendId;
	/** Connection string */
	FString ConnectionURL;
	/** User initiating the request */
	int32 LocalUserNum;

	/** Hidden on purpose */
	FOnlineAsyncEventSteamInviteAccepted() = delete;

public:

	FOnlineAsyncEventSteamInviteAccepted(FOnlineSubsystemSteam* InSubsystem, const FUniqueNetIdSteam& InFriendId, const FString& InConnectionURL) :
	  FOnlineAsyncEvent(InSubsystem),
	  FriendId(FUniqueNetIdSteam::EmptyId()),
	  ConnectionURL(InConnectionURL),
	  LocalUserNum(0)
	{
	}

	~FOnlineAsyncEventSteamInviteAccepted() 
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncEventSteamInviteAccepted Friend: %s URL: %s"), *FriendId->ToDebugString(), *ConnectionURL);
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override;
};
