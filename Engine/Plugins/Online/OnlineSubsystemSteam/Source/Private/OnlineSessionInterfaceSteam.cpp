// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSessionInterfaceSteam.h"
#include "Misc/CommandLine.h"
#include "Online/OnlineBase.h"
#include "UObject/CoreNet.h"
#include "Online/OnlineSessionNames.h"
#include "SocketSubsystem.h"
#include "OnlineSessionAsyncLobbySteam.h"
#include "OnlineSessionAsyncServerSteam.h"
#include "OnlineLeaderboardInterfaceSteam.h"
#include "OnlineAuthInterfaceSteam.h"
#include "Online/LANBeacon.h"
#include "NboSerializerSteam.h"
#include "Interfaces/VoiceInterface.h"
#include <steam/steamuniverse.h>

#if WITH_ENGINE
#include "Engine/EngineBaseTypes.h"
#endif


/** Constructor for non-lobby sessions */
FOnlineSessionInfoSteam::FOnlineSessionInfoSteam(ESteamSession::Type InSessionType) :
	SessionType(InSessionType),
	HostAddr(nullptr),
	SteamP2PAddr(nullptr),
	SessionId(FUniqueNetIdSteam::Create((uint64)0)),
	ConnectionMethod((InSessionType == ESteamSession::LANSession) ? FSteamConnectionMethod::Direct : FSteamConnectionMethod::None)
{
}

/** Constructor for sessions that represent a Steam lobby or server */
FOnlineSessionInfoSteam::FOnlineSessionInfoSteam(ESteamSession::Type InSessionType, const FUniqueNetIdSteam& InSessionId) :
	SessionType(InSessionType),
	HostAddr(nullptr),
	SteamP2PAddr(nullptr),
	SessionId(InSessionId.AsShared()),
	ConnectionMethod(FSteamConnectionMethod::None)
{
}

void FOnlineSessionInfoSteam::Init()
{
}

void FOnlineSessionInfoSteam::InitLAN()
{
#if WITH_ENGINE
	SessionType = ESteamSession::LANSession;
	ConnectionMethod = FSteamConnectionMethod::Direct;

	uint64 Nonce = 0;
	GenerateNonce((uint8*)&Nonce, 8);
	SessionId = FUniqueNetIdSteam::Create(Nonce);

	// Read the IP from the system
	bool bCanBindAll;
	HostAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalHostAddr(*GLog, bCanBindAll);
	// Now set the port that was configured
	HostAddr->SetPort(FURL::UrlConfig.DefaultPort);

	Init();
#else //WITH_ENGINE
	ensure(false); //Not supported on non-engine
#endif //WITH_ENGINE
}

/**
 *	Async task for ending a Steam online session
 */
class FOnlineAsyncTaskSteamEndSession : public FOnlineAsyncTaskSteam
{
private:
	/** Name of session ending */
	FName SessionName;

public:
	FOnlineAsyncTaskSteamEndSession(FOnlineSubsystemSteam* InSubsystem, FName InSessionName) :
		FOnlineAsyncTaskSteam(InSubsystem, k_uAPICallInvalid),
		SessionName(InSessionName)
	{
	}

	~FOnlineAsyncTaskSteamEndSession()
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncTaskSteamEndSession bWasSuccessful: %d SessionName: %s"), WasSuccessful(), *SessionName.ToString());
	}

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override
	{
		bIsComplete = true;
		bWasSuccessful = true;
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override
	{
		IOnlineSessionPtr SessionInt = Subsystem->GetSessionInterface();
		FNamedOnlineSession* Session = SessionInt->GetNamedSession(SessionName);
		if (Session)
		{
			Session->SessionState = EOnlineSessionState::Ended;
		}
	}

	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override
	{
		IOnlineSessionPtr SessionInt = Subsystem->GetSessionInterface();
		if (SessionInt.IsValid())
		{
			SessionInt->TriggerOnEndSessionCompleteDelegates(SessionName, bWasSuccessful);
		}
	}
};

/**
 *	Async task for destroying a Steam online session
 */
class FOnlineAsyncTaskSteamDestroySession : public FOnlineAsyncTaskSteam
{
private:
	/** Name of session ending */
	FName SessionName;
	/** */
	FOnDestroySessionCompleteDelegate CompletionDelegate;

public:
	FOnlineAsyncTaskSteamDestroySession(FOnlineSubsystemSteam* InSubsystem, FName InSessionName, const FOnDestroySessionCompleteDelegate& InCompletionDelegate) :
		FOnlineAsyncTaskSteam(InSubsystem, k_uAPICallInvalid),
		SessionName(InSessionName),
		CompletionDelegate(InCompletionDelegate)
	{
	}

	~FOnlineAsyncTaskSteamDestroySession()
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncTaskSteamDestroySession bWasSuccessful: %d SessionName: %s"), WasSuccessful(), *SessionName.ToString());
	}

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override
	{
		bIsComplete = true;
		bWasSuccessful = true;
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override
	{
		IOnlineSessionPtr SessionInt = Subsystem->GetSessionInterface();
		if (SessionInt.IsValid())
		{
			FNamedOnlineSession* Session = SessionInt->GetNamedSession(SessionName);
			if (Session)
			{
				SessionInt->RemoveNamedSession(SessionName);
				if (SessionInt->GetNumSessions() == 0)
				{
					IOnlineVoicePtr VoiceInt = Subsystem->GetVoiceInterface();
					if (VoiceInt.IsValid())
					{
						if (!Subsystem->IsDedicated())
						{
							// Stop local talkers
							VoiceInt->UnregisterLocalTalkers();
						}

						// Stop remote voice 
						VoiceInt->RemoveAllRemoteTalkers();
					}

					FOnlineAuthSteamPtr AuthInt = Subsystem->GetAuthInterface();
					if (AuthInt.IsValid())
					{
						AuthInt->RevokeAllTickets();
					}
				}
			}
		}
	}

	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override
	{
		IOnlineSessionPtr SessionInt = Subsystem->GetSessionInterface();
		if (SessionInt.IsValid())
		{
			CompletionDelegate.ExecuteIfBound(SessionName, bWasSuccessful);
			SessionInt->TriggerOnDestroySessionCompleteDelegates(SessionName, bWasSuccessful);
		}
	}
};

bool FOnlineSessionSteam::CreateSession(int32 HostingPlayerNum, FName SessionName, const FOnlineSessionSettings& NewSessionSettings)
{
	// In Steam, bUsesPresence and bUseLobbiesIfAvailable have equivalent meaning and should have the same value
	if (NewSessionSettings.bUsesPresence != NewSessionSettings.bUseLobbiesIfAvailable)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[%hs] The values of FOnlineSessionSettings::bUsesPresence and FOnlineSessionSettings::bUseLobbiesIfAvailable are treated as equal and have to match"), __FUNCTION__);
		TriggerOnCreateSessionCompleteDelegates(SessionName, false);
		return false;
	}

	uint32 Result = ONLINE_FAIL;

	// Check for an existing session
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session == nullptr)
	{
		// Create a new session and deep copy the game settings
		Session = AddNamedSession(SessionName, NewSessionSettings);
		check(Session);
		Session->SessionState = EOnlineSessionState::Creating;
		Session->NumOpenPrivateConnections = NewSessionSettings.NumPrivateConnections;
		Session->NumOpenPublicConnections = NewSessionSettings.bIsDedicated ? NewSessionSettings.NumPublicConnections : NewSessionSettings.NumPublicConnections - 1;

		Session->HostingPlayerNum = HostingPlayerNum;
		Session->OwningUserId = SteamUser() ? FUniqueNetIdSteamPtr(FUniqueNetIdSteam::Create(SteamUser()->GetSteamID())) : nullptr;
		Session->OwningUserName = SteamFriends() ? UTF8_TO_TCHAR(SteamFriends()->GetPersonaName()) : GetCustomDedicatedServerName();
		
		// Unique identifier of this build for compatibility
		Session->SessionSettings.BuildUniqueId = GetBuildUniqueId();

		// Create Internet or LAN match
		if (!NewSessionSettings.bIsLANMatch)
		{
			if (Session->SessionSettings.bUseLobbiesIfAvailable)
			{
				Result = CreateLobbySession(HostingPlayerNum, Session);
			}
			else
			{
				Result = CreateInternetSession(HostingPlayerNum, Session);
			}
		}
		else
		{
			Result = CreateLANSession(HostingPlayerNum, Session);
		}

		if (Result != ONLINE_IO_PENDING)
		{
			// Set the game state as pending (not started)
			Session->SessionState = EOnlineSessionState::Pending;

			if (Result != ONLINE_SUCCESS)
			{
				// Clean up the session info so we don't get into a confused state
				RemoveNamedSession(SessionName);
			}
			else
			{
				RegisterLocalPlayers(Session);
			}
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot create session '%s': session already exists."), *SessionName.ToString());
	}

	if (Result != ONLINE_IO_PENDING)
	{
		TriggerOnCreateSessionCompleteDelegates(SessionName, (Result == ONLINE_SUCCESS) ? true : false);
	}
	
	return Result == ONLINE_IO_PENDING || Result == ONLINE_SUCCESS;
}

bool FOnlineSessionSteam::CreateSession(const FUniqueNetId& HostingPlayerId, FName SessionName, const FOnlineSessionSettings& NewSessionSettings)
{
	// @todo: use proper HostingPlayerId
	return CreateSession(0, SessionName, NewSessionSettings);
}

uint32 FOnlineSessionSteam::CreateLobbySession(int32 HostingPlayerNum, FNamedOnlineSession* Session)
{
	uint32 Result = ONLINE_FAIL; 

	if (Session)
	{
		/** Max lobby size is sum of private/public (@TODO ONLINE - not sure we can differentiate) */
		int32 MaxLobbySize = Session->SessionSettings.NumPrivateConnections + Session->SessionSettings.NumPublicConnections;

		/** Generate the proper lobby type from our session settings */
		ELobbyType SteamLobbyType = BuildLobbyType(&Session->SessionSettings);

		FOnlineAsyncTaskSteamCreateLobby* NewTask = new FOnlineAsyncTaskSteamCreateLobby(SteamSubsystem, Session->SessionName, SteamLobbyType, MaxLobbySize);
		SteamSubsystem->QueueAsyncTask(NewTask);

		Result = ONLINE_IO_PENDING;
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("CreateLobbySession: NULL Session"));
	}

	return Result;
}

uint32 FOnlineSessionSteam::CreateInternetSession(int32 HostingPlayerNum, FNamedOnlineSession* Session)
{
	uint32 Result = ONLINE_FAIL;

	// Only allowed one published session with Steam
	FNamedOnlineSession* ListedSession = GetGameServerSession();
	if (ListedSession == nullptr)
	{
		if (SteamSubsystem->IsSteamServerAvailable())
		{
			// Reset the policy response
			bPolicyResponseReceived = false;

			FOnlineAsyncTaskSteamCreateServer* NewTask = new FOnlineAsyncTaskSteamCreateServer(SteamSubsystem, Session->SessionName);
			SteamSubsystem->QueueAsyncTask(NewTask);
			Result = ONLINE_IO_PENDING;
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to initialize game server with Steam!"));
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("Advertised session %s already exists, unable to create another."), *Session->SessionName.ToString());
	}

	return Result;
}


uint32 FOnlineSessionSteam::CreateLANSession(int32 HostingPlayerNum, FNamedOnlineSession* Session)
{
	check(Session);
	uint32 Result = ONLINE_SUCCESS;

	// Setup the host session info
	FOnlineSessionInfoSteam* NewSessionInfo = new FOnlineSessionInfoSteam(ESteamSession::LANSession);
	NewSessionInfo->InitLAN();

	if (!Session->OwningUserId.IsValid())
	{
		// Use the lan user id, requires us to advertise the host ip and port
		Session->OwningUserId = FUniqueNetIdSteam::Create(k_steamIDLanModeGS);
	}
	Session->SessionInfo = MakeShareable(NewSessionInfo);

	// Don't create the LAN beacon if advertising is off
	if (Session->SessionSettings.bShouldAdvertise)
	{
		if (!LANSession)
		{
			LANSession = new FLANSession();
		}
		
		FOnValidQueryPacketDelegate QueryPacketDelegate = FOnValidQueryPacketDelegate::CreateRaw(this, &FOnlineSessionSteam::OnValidQueryPacketReceived);
		if (!LANSession->Host(QueryPacketDelegate))
		{
			Result = ONLINE_FAIL;
		}
	}

	return Result;
}

bool FOnlineSessionSteam::StartSession(FName SessionName)
{
	uint32 Result = ONLINE_FAIL;
	// Grab the session information by name
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		// Can't start a match multiple times
		if (Session->SessionState == EOnlineSessionState::Pending ||
			Session->SessionState == EOnlineSessionState::Ended)
		{
			if (!Session->SessionSettings.bIsLANMatch)
			{
				Result = ONLINE_SUCCESS;
				Session->SessionState = EOnlineSessionState::InProgress;

				if (SteamFriends() != nullptr)
				{
					for (int32 PlayerIdx=0; PlayerIdx < Session->RegisteredPlayers.Num(); PlayerIdx++)
					{
						FUniqueNetIdSteam& Player = (FUniqueNetIdSteam&)Session->RegisteredPlayers[PlayerIdx].Get();
						SteamFriends()->SetPlayedWith(Player);
					}
				}
			}
			else
			{
				// If this lan match has join in progress disabled, shut down the beacon
				if (!Session->SessionSettings.bAllowJoinInProgress)
				{
					LANSession->StopLANSession();
				}
				Result = ONLINE_SUCCESS;
				Session->SessionState = EOnlineSessionState::InProgress;
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning,	TEXT("Can't start an online session (%s) in state %s"),
				*SessionName.ToString(),
				EOnlineSessionState::ToString(Session->SessionState));
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't start an online game for session (%s) that hasn't been created"), *SessionName.ToString());
	}

	if (Result != ONLINE_IO_PENDING)
	{
		// Just trigger the delegate
		TriggerOnStartSessionCompleteDelegates(SessionName, (Result == ONLINE_SUCCESS) ? true : false);
	}

	return Result == ONLINE_SUCCESS || Result == ONLINE_IO_PENDING;
}

bool FOnlineSessionSteam::UpdateSession(FName SessionName, FOnlineSessionSettings& UpdatedSessionSettings, bool bShouldRefreshOnlineData)
{
	bool bWasSuccessful = true;

	// Grab the session information by name
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		if (!Session->SessionSettings.bIsLANMatch)
		{
			FOnlineSessionInfoSteam* SessionInfo = (FOnlineSessionInfoSteam*)(Session->SessionInfo.Get());
			if (SessionInfo)
			{
				if (SessionInfo->SessionType == ESteamSession::LobbySession && SessionInfo->SessionId->IsValid())
				{
					// Lobby update
					FOnlineAsyncTaskSteamUpdateLobby* NewTask = new FOnlineAsyncTaskSteamUpdateLobby(SteamSubsystem, SessionName, bShouldRefreshOnlineData, UpdatedSessionSettings);
					SteamSubsystem->QueueAsyncTask(NewTask);
				}
				else if (SessionInfo->SessionType == ESteamSession::AdvertisedSessionHost)
				{
					// Gameserver update
					FOnlineAsyncTaskSteamUpdateServer* NewTask = new FOnlineAsyncTaskSteamUpdateServer(SteamSubsystem, SessionName, bShouldRefreshOnlineData, UpdatedSessionSettings);
					SteamSubsystem->QueueAsyncTask(NewTask);
				}
			}
			else
			{
				bWasSuccessful = false;
			}
		}
		else
		{
			// @TODO ONLINE update LAN settings
			Session->SessionSettings = UpdatedSessionSettings;
			TriggerOnUpdateSessionCompleteDelegates(SessionName, bWasSuccessful);
		}
	}

	return bWasSuccessful;
}

bool FOnlineSessionSteam::EndSession(FName SessionName)
{
	uint32 Result = ONLINE_FAIL;

	// Grab the session information by name
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		// Can't end a match that isn't in progress
		if (Session->SessionState == EOnlineSessionState::InProgress)
		{
			if (!Session->SessionSettings.bIsLANMatch)
			{
				Result = EndInternetSession(Session);
			}
			else
			{
				// If the session should be advertised and the lan beacon was destroyed, recreate
				if (Session->SessionSettings.bShouldAdvertise && 
					LANSession->LanBeacon == nullptr &&
					SteamSubsystem->IsServer())
				{
					// Recreate the beacon
					Result = CreateLANSession(Session->HostingPlayerNum, Session);
				}
				else
				{
					Result = ONLINE_SUCCESS;
				}
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't end session (%s) in state %s"),
				*SessionName.ToString(),
				EOnlineSessionState::ToString(Session->SessionState));
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't end an online game for session (%s) that hasn't been created"),
			*SessionName.ToString());
	}

	if (Result != ONLINE_IO_PENDING)
	{
		if (Session)
		{
			Session->SessionState = EOnlineSessionState::Ended;
		}

		TriggerOnEndSessionCompleteDelegates(SessionName, (Result == ONLINE_SUCCESS) ? true : false);
	}

	return Result == ONLINE_SUCCESS || Result == ONLINE_IO_PENDING;
}

uint32 FOnlineSessionSteam::EndInternetSession(FNamedOnlineSession* Session)
{
	// Only called from EndSession/DestroySession and presumes only in InProgress state
	check(Session && Session->SessionState == EOnlineSessionState::InProgress);

	// Enqueue a flush leaderboard on async task list
	FOnlineLeaderboardsSteamPtr Leaderboards = StaticCastSharedPtr<FOnlineLeaderboardsSteam>(SteamSubsystem->GetLeaderboardsInterface());
	if (Leaderboards.IsValid())
	{
		Leaderboards->FlushLeaderboards(Session->SessionName);
	}

	Session->SessionState = EOnlineSessionState::Ending;

	// Guaranteed to be called after the flush is complete
	FOnlineAsyncTaskSteamEndSession* NewTask = new FOnlineAsyncTaskSteamEndSession(SteamSubsystem,	Session->SessionName);
	SteamSubsystem->QueueAsyncTask(NewTask);

	return ONLINE_IO_PENDING;
}

bool FOnlineSessionSteam::DestroySession(FName SessionName, const FOnDestroySessionCompleteDelegate& CompletionDelegate)
{
	uint32 Result = ONLINE_FAIL;
	// Find the session in question
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		if (Session->SessionState != EOnlineSessionState::Destroying)
		{
			if (!Session->SessionSettings.bIsLANMatch)
			{
				if (Session->SessionState == EOnlineSessionState::InProgress)
				{
					// Enqueue all the end session tasks first
					EndInternetSession(Session);
				}

				if (Session->SessionSettings.bUseLobbiesIfAvailable)
				{
					Result = DestroyLobbySession(Session, CompletionDelegate);
				}
				else
				{
					Result = DestroyInternetSession(Session, CompletionDelegate);
				}
			}
			else
			{
				if (LANSession)
				{
					// Tear down the LAN beacon
					LANSession->StopLANSession();
					delete LANSession;
					LANSession = nullptr;
				}

				Result = ONLINE_SUCCESS;
			}

			if (Result != ONLINE_IO_PENDING)
			{
				// The session info is no longer needed
				RemoveNamedSession(Session->SessionName);
				CompletionDelegate.ExecuteIfBound(SessionName, (Result == ONLINE_SUCCESS) ? true : false);
				TriggerOnDestroySessionCompleteDelegates(SessionName, (Result == ONLINE_SUCCESS) ? true : false);
			}
		}
		else
		{
			// Purposefully skip the delegate call as one should already be in flight
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Already in process of destroying session (%s)"), *SessionName.ToString());
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't destroy a null online session (%s)"), *SessionName.ToString());
		CompletionDelegate.ExecuteIfBound(SessionName, false);
		TriggerOnDestroySessionCompleteDelegates(SessionName, false);
	}

	return Result == ONLINE_SUCCESS || Result == ONLINE_IO_PENDING;
}

uint32 FOnlineSessionSteam::DestroyLobbySession(FNamedOnlineSession* Session, const FOnDestroySessionCompleteDelegate& CompletionDelegate)
{
	Session->SessionState = EOnlineSessionState::Destroying;

	if (Session->SessionInfo.IsValid())
	{
		FOnlineSessionInfoSteam* SessionInfo = (FOnlineSessionInfoSteam*)(Session->SessionInfo.Get());
		check(SessionInfo->SessionType == ESteamSession::LobbySession);

		FOnlineAsyncTaskSteamLeaveLobby* NewTask = new FOnlineAsyncTaskSteamLeaveLobby(SteamSubsystem, Session->SessionName, *SessionInfo->SessionId);
		SteamSubsystem->QueueAsyncTask(NewTask);
	}

	FOnlineAsyncTaskSteamDestroySession* NewTask = new FOnlineAsyncTaskSteamDestroySession(SteamSubsystem, Session->SessionName, CompletionDelegate);
	SteamSubsystem->QueueAsyncTask(NewTask);

	return ONLINE_IO_PENDING;
}

uint32 FOnlineSessionSteam::DestroyInternetSession(FNamedOnlineSession* Session, const FOnDestroySessionCompleteDelegate& CompletionDelegate)
{
	Session->SessionState = EOnlineSessionState::Destroying;

	if (Session->SessionInfo.IsValid())
	{
		FOnlineSessionInfoSteam* SessionInfo = (FOnlineSessionInfoSteam*)(Session->SessionInfo.Get());
		check(SessionInfo->SessionType == ESteamSession::AdvertisedSessionHost || SessionInfo->SessionType == ESteamSession::AdvertisedSessionClient);
	}

	// Clear any session advertisements this account had with this session.
	FOnlineAuthSteamPtr SteamAuth = SteamSubsystem->GetAuthInterface();
	if (SteamUser() != nullptr && SteamUser()->BLoggedOn() && (!SteamAuth.IsValid() || !SteamAuth->IsSessionAuthEnabled()))
	{
		UE_LOG_ONLINE(Warning, TEXT("AUTH: DestroyInternetSession is calling the depricated AdvertiseGame call"));
		SteamUser()->AdvertiseGame(k_steamIDNil, 0, 0);
	}

	if (bSteamworksGameServerConnected && GameServerSteamId->IsValid())
	{
		// Remove this server from the server list
		FOnlineAsyncTaskSteamLogoffServer* LogoffTask = new FOnlineAsyncTaskSteamLogoffServer(SteamSubsystem, Session->SessionName);
		SteamSubsystem->QueueAsyncTask(LogoffTask);
	}

	// Destroy the session
	FOnlineAsyncTaskSteamDestroySession* DestroyTask = new FOnlineAsyncTaskSteamDestroySession(SteamSubsystem, Session->SessionName, CompletionDelegate);
	SteamSubsystem->QueueAsyncTask(DestroyTask);

	return ONLINE_IO_PENDING;
}

bool FOnlineSessionSteam::IsPlayerInSession(FName SessionName, const FUniqueNetId& UniqueId)
{
	return IsPlayerInSessionImpl(this, SessionName, UniqueId);
}

bool FOnlineSessionSteam::StartMatchmaking(const TArray< FUniqueNetIdRef >& LocalPlayers, FName SessionName, const FOnlineSessionSettings& NewSessionSettings, TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	UE_LOG_ONLINE_SESSION(Warning, TEXT("StartMatchmaking is not supported on this platform. Use FindSessions or FindSessionById."));
	TriggerOnMatchmakingCompleteDelegates(SessionName, false);
	return false;
}

bool FOnlineSessionSteam::CancelMatchmaking(int32 SearchingPlayerNum, FName SessionName)
{
	UE_LOG_ONLINE_SESSION(Warning, TEXT("CancelMatchmaking is not supported on this platform. Use CancelFindSessions."));
	TriggerOnCancelMatchmakingCompleteDelegates(SessionName, false);
	return false;
}

bool FOnlineSessionSteam::CancelMatchmaking(const FUniqueNetId& SearchingPlayerId, FName SessionName)
{
	UE_LOG_ONLINE_SESSION(Warning, TEXT("CancelMatchmaking is not supported on this platform. Use CancelFindSessions."));
	TriggerOnCancelMatchmakingCompleteDelegates(SessionName, false);
	return false;
}

bool FOnlineSessionSteam::FindSessions(int32 SearchingPlayerNum, const TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	uint32 Return = ONLINE_FAIL;

	// Dedicated servers shouldn't be matchmaking. 
	if (SteamSubsystem->IsDedicated())
	{
		SearchSettings->SearchState = EOnlineAsyncTaskState::Failed;
		TriggerOnFindSessionsCompleteDelegates(false);
		return false;
	}

	// Don't start another search while one is in progress
	if (!CurrentSessionSearch.IsValid() && SearchSettings->SearchState != EOnlineAsyncTaskState::InProgress)
	{
		// Free up previous results
		SearchSettings->SearchResults.Empty();

		// Copy the search pointer so we can keep it around
		CurrentSessionSearch = SearchSettings;

		// Check if its a LAN query
		if (SearchSettings->bIsLanQuery == false)
		{
			Return = FindInternetSession(SearchSettings);
		}
		else
		{
			Return = FindLANSession(SearchSettings);
		}

		if (Return == ONLINE_IO_PENDING)
		{
			SearchSettings->SearchState = EOnlineAsyncTaskState::InProgress;
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Ignoring game search request while one is pending"));
		Return = ONLINE_IO_PENDING;
	}

	return Return == ONLINE_SUCCESS || Return == ONLINE_IO_PENDING;
}

bool FOnlineSessionSteam::FindSessions(const FUniqueNetId& SearchingPlayerId, const TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	// @todo: use proper SearchingPlayerId
	return FindSessions(0, SearchSettings);
}

bool FOnlineSessionSteam::FindSessionById(const FUniqueNetId& SearchingUserId, const FUniqueNetId& SessionId, const FUniqueNetId& FriendId, const FOnSingleSessionResultCompleteDelegate& CompletionDelegates)
{
	FOnlineSessionSearchResult EmptyResult;
	CompletionDelegates.ExecuteIfBound(0, false, EmptyResult);
	return true;
}

uint32 FOnlineSessionSteam::FindInternetSession(const TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	bool PresenceSearch = false;
	bool LobbySearch = false;
	if (SearchSettings->QuerySettings.Get(SEARCH_LOBBIES, LobbySearch) && LobbySearch)
	{
		FOnlineAsyncTaskSteamFindLobbies* NewTask = new FOnlineAsyncTaskSteamFindLobbies(SteamSubsystem, SearchSettings);
		SteamSubsystem->QueueAsyncTask(NewTask);
	}
	else
	{
		FOnlineAsyncTaskSteamFindServers* NewTask = new FOnlineAsyncTaskSteamFindServers(SteamSubsystem, SearchSettings, OnFindSessionsCompleteDelegates);
		SteamSubsystem->QueueAsyncTask(NewTask);
	}

	return ONLINE_IO_PENDING;
}

uint32 FOnlineSessionSteam::FindLANSession(const TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	uint32 Return = ONLINE_IO_PENDING;

	if (!LANSession)
	{
		LANSession = new FLANSession();
	}

	// Recreate the unique identifier for this client
	GenerateNonce((uint8*)&LANSession->LanNonce, 8);

	FOnValidResponsePacketDelegate ResponseDelegate = FOnValidResponsePacketDelegate::CreateRaw(this, &FOnlineSessionSteam::OnValidResponsePacketReceived);
	FOnSearchingTimeoutDelegate TimeoutDelegate = FOnSearchingTimeoutDelegate::CreateRaw(this, &FOnlineSessionSteam::OnLANSearchTimeout);

	FNboSerializeToBufferSteam Packet(LAN_BEACON_MAX_PACKET_SIZE);
	LANSession->CreateClientQueryPacket(Packet, LANSession->LanNonce);
	if (Packet.HasOverflow() || LANSession->Search(Packet, ResponseDelegate, TimeoutDelegate) == false)
	{
		Return = ONLINE_FAIL;
		delete LANSession;
		LANSession = nullptr;

		CurrentSessionSearch->SearchState = EOnlineAsyncTaskState::Failed;

		// Just trigger the delegate as having failed
		TriggerOnFindSessionsCompleteDelegates(false);
	}

	return Return;
}

bool FOnlineSessionSteam::CancelFindSessions()
{
	uint32 Return = ONLINE_FAIL;
	if (CurrentSessionSearch.IsValid() && CurrentSessionSearch->SearchState == EOnlineAsyncTaskState::InProgress)
	{
		// Make sure it's the right type
		if (CurrentSessionSearch->bIsLanQuery)
		{
			Return = ONLINE_SUCCESS;
			LANSession->StopLANSession();
			CurrentSessionSearch->SearchState = EOnlineAsyncTaskState::Failed;
		}
		else
		{
			// @TODO ONLINE Server list version
			Return = ONLINE_SUCCESS;
			// There is no CANCEL lobby query
			// NULLing out the object will prevent the async event from adding the results
			CurrentSessionSearch->SearchState = EOnlineAsyncTaskState::Failed;
			CurrentSessionSearch = nullptr;
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't cancel a search that isn't in progress"));
	}

	if (Return != ONLINE_IO_PENDING)
	{
		TriggerOnCancelFindSessionsCompleteDelegates(true);
	}

	return Return == ONLINE_SUCCESS || Return == ONLINE_IO_PENDING;
}

bool FOnlineSessionSteam::JoinSession(int32 PlayerNum, FName SessionName, const FOnlineSessionSearchResult& DesiredSession)
{
	// In Steam, bUsesPresence and bUseLobbiesIfAvailable have equivalent meaning and should have the same value
	if(DesiredSession.Session.SessionSettings.bUsesPresence != DesiredSession.Session.SessionSettings.bUseLobbiesIfAvailable)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[%hs] The values of FOnlineSessionSettings::bUsesPresence and FOnlineSessionSettings::bUseLobbiesIfAvailable are treated as equal and have to match"), __FUNCTION__);
		TriggerOnJoinSessionCompleteDelegates(SessionName, EOnJoinSessionCompleteResult::UnknownError);
		return false;
	}

	uint32 Return = ONLINE_FAIL;
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	// Don't join a session if already in one or hosting one
	if (Session == nullptr)
	{
		// Create a named session from the search result data
		Session = AddNamedSession(SessionName, DesiredSession.Session);
		Session->HostingPlayerNum = PlayerNum;

		// Create Internet or LAN match
		if (!Session->SessionSettings.bIsLANMatch)
		{
			if (DesiredSession.Session.SessionInfo.IsValid())
			{
				const FOnlineSessionInfoSteam* SearchSessionInfo = (const FOnlineSessionInfoSteam*)DesiredSession.Session.SessionInfo.Get();

				if (DesiredSession.Session.SessionSettings.bUseLobbiesIfAvailable)
				{
					FOnlineSessionInfoSteam* NewSessionInfo = new FOnlineSessionInfoSteam(ESteamSession::LobbySession, *SearchSessionInfo->SessionId);
					Session->SessionInfo = MakeShareable(NewSessionInfo);

					Return = JoinLobbySession(PlayerNum, Session, &DesiredSession.Session);
				}
				else
				{
					FOnlineSessionInfoSteam* NewSessionInfo = new FOnlineSessionInfoSteam(ESteamSession::AdvertisedSessionClient, *SearchSessionInfo->SessionId);
					Session->SessionInfo = MakeShareable(NewSessionInfo);

					Return = JoinInternetSession(PlayerNum, Session, &DesiredSession.Session);
				}
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Invalid session info on search result"));
			}
		}
		else
		{
			FOnlineSessionInfoSteam* NewSessionInfo = new FOnlineSessionInfoSteam(ESteamSession::LANSession);
			Session->SessionInfo = MakeShareable(NewSessionInfo);

			Return = JoinLANSession(PlayerNum, Session, &DesiredSession.Session);
		}

		if (Return != ONLINE_IO_PENDING)
		{
			if (Return != ONLINE_SUCCESS)
			{
				// Clean up the session info so we don't get into a confused state
				RemoveNamedSession(SessionName);
			}
			else
			{
				RegisterLocalPlayers(Session);
			}
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Session (%s) already exists, can't join twice"), *SessionName.ToString());
	}

	if (Return != ONLINE_IO_PENDING)
	{
		// Just trigger the delegate as having failed
		TriggerOnJoinSessionCompleteDelegates(SessionName, Return == ONLINE_SUCCESS ? EOnJoinSessionCompleteResult::Success : EOnJoinSessionCompleteResult::UnknownError);
	}

	return Return == ONLINE_SUCCESS || Return == ONLINE_IO_PENDING;
}

bool FOnlineSessionSteam::JoinSession(const FUniqueNetId& PlayerId, FName SessionName, const FOnlineSessionSearchResult& DesiredSession)
{
	// @todo: use proper PlayerId
	return JoinSession(0, SessionName, DesiredSession);
}

uint32 FOnlineSessionSteam::JoinLobbySession(int32 PlayerNum, FNamedOnlineSession* Session, const FOnlineSession* SearchSession)
{
	uint32 Result = ONLINE_FAIL;
	if (Session->SessionInfo.IsValid())
	{
		FOnlineSessionInfoSteam* SteamSessionInfo = (FOnlineSessionInfoSteam*)(Session->SessionInfo.Get());
		if (SteamSessionInfo->SessionType == ESteamSession::LobbySession && SteamSessionInfo->SessionId->IsValid())
		{
			// Copy the session info over
			const FOnlineSessionInfoSteam* SearchSessionInfo = (const FOnlineSessionInfoSteam*)SearchSession->SessionInfo.Get();
			SteamSessionInfo->HostAddr = SearchSessionInfo->HostAddr;
			SteamSessionInfo->SteamP2PAddr = SearchSessionInfo->SteamP2PAddr;
			SteamSessionInfo->ConnectionMethod = SearchSessionInfo->ConnectionMethod;

			// The settings found on the search object will be duplicated again when we enter the lobby, possibly updated
			FOnlineAsyncTaskSteamJoinLobby* NewTask = new FOnlineAsyncTaskSteamJoinLobby(SteamSubsystem, Session->SessionName, *SteamSessionInfo->SessionId);
			SteamSubsystem->QueueAsyncTask(NewTask);
			Result = ONLINE_IO_PENDING;
		}
	}

	return Result;
}

uint32 FOnlineSessionSteam::JoinInternetSession(int32 PlayerNum, FNamedOnlineSession* Session, const FOnlineSession* SearchSession)
{
	uint32 Result = ONLINE_FAIL;
	Session->SessionState = EOnlineSessionState::Pending;

	if (Session->SessionInfo.IsValid())
	{
		FOnlineSessionInfoSteam* SteamSessionInfo = (FOnlineSessionInfoSteam*)(Session->SessionInfo.Get());
		if (SteamSessionInfo->SessionType == ESteamSession::AdvertisedSessionClient && SteamSessionInfo->SessionId->IsValid())
		{
			// Copy the session info over
			const FOnlineSessionInfoSteam* SearchSessionInfo = (const FOnlineSessionInfoSteam*)SearchSession->SessionInfo.Get();
			SteamSessionInfo->HostAddr = SearchSessionInfo->HostAddr;
			SteamSessionInfo->SteamP2PAddr = SearchSessionInfo->SteamP2PAddr;
			SteamSessionInfo->ConnectionMethod = SearchSessionInfo->ConnectionMethod;

			if (SearchSession->SessionSettings.bAllowJoinViaPresence)
			{
				FString ConnectionString = GetSteamConnectionString(Session->SessionName);
				if (!SteamFriends()->SetRichPresence("connect", TCHAR_TO_UTF8(*ConnectionString)))
				{
					UE_LOG_ONLINE_SESSION(Verbose, TEXT("Failed to set rich presence for session %s"), *Session->SessionName.ToString());
				}

				// SteamAuth will auto advertise any sessions we join.
				bool bShouldUseFallback = true;
				FOnlineAuthSteamPtr SteamAuth = SteamSubsystem->GetAuthInterface();
				if (SteamAuth.IsValid() && SteamAuth->IsSessionAuthEnabled())
				{
					bShouldUseFallback = false;
				}

				// Advertise any servers we join (However, if we're using SteamAuth [as determined above], then we should not do this)
				if (SteamUser() != nullptr && SteamUser()->BLoggedOn() && bShouldUseFallback)
				{
					uint32 IpAddr;
					uint32 Port = SteamSessionInfo->HostAddr->GetPort();
					SteamSessionInfo->HostAddr->GetIp(IpAddr);
					SteamUser()->AdvertiseGame(k_steamIDNonSteamGS, IpAddr, Port);
					UE_LOG_ONLINE(Warning, TEXT("AUTH: JoinInternetSession is calling the depricated AdvertiseGame call"));
				}
			}
			Result = ONLINE_SUCCESS;
		}
	}

	return Result;
}

uint32 FOnlineSessionSteam::JoinLANSession(int32 PlayerNum, FNamedOnlineSession* Session, const FOnlineSession* SearchSession)
{
	uint32 Result = ONLINE_FAIL;
	Session->SessionState = EOnlineSessionState::Pending;

	if (Session->SessionInfo.IsValid())
	{
		// Copy the session info over
		const FOnlineSessionInfoSteam* SearchSessionInfo = (const FOnlineSessionInfoSteam*)SearchSession->SessionInfo.Get();
		FOnlineSessionInfoSteam* SessionInfo = (FOnlineSessionInfoSteam*)Session->SessionInfo.Get();

		SessionInfo->HostAddr = SearchSessionInfo->HostAddr->Clone();
		SessionInfo->ConnectionMethod = FSteamConnectionMethod::Direct;
		Result = ONLINE_SUCCESS;
	}

	return Result;
}

bool FOnlineSessionSteam::FindFriendSession(int32 LocalUserNum, const FUniqueNetId& Friend)
{
	bool bSuccess = false;

	const FUniqueNetIdSteam& SteamFriendId = (const FUniqueNetIdSteam&) Friend;

	// Don't start another search while one is in progress
	if (!CurrentSessionSearch.IsValid())
	{
		FriendGameInfo_t FriendGameInfo;
		if (SteamFriends()->GetFriendGamePlayed(SteamFriendId, &FriendGameInfo))
		{
			if (FriendGameInfo.m_gameID.AppID() == SteamSubsystem->GetSteamAppId())
			{
				// Create a search settings object 
				TSharedRef<FOnlineSessionSearch> SearchSettings = MakeShareable(new FOnlineSessionSearch());
				CurrentSessionSearch = SearchSettings;
				CurrentSessionSearch->SearchState = EOnlineAsyncTaskState::InProgress;

				if (FriendGameInfo.m_steamIDLobby.IsValid())
				{
					const FUniqueNetIdSteamRef LobbyId = FUniqueNetIdSteam::Create(FriendGameInfo.m_steamIDLobby);
					FOnlineAsyncTaskSteamFindLobbiesForFriendSession* NewTask = new FOnlineAsyncTaskSteamFindLobbiesForFriendSession(SteamSubsystem, *LobbyId, CurrentSessionSearch, LocalUserNum, OnFindFriendSessionCompleteDelegates[LocalUserNum]);
					SteamSubsystem->QueueAsyncTask(NewTask);
					bSuccess = true;
				}
				else
				{
					// Search for the session via host ip
#if WITH_ENGINE
					TSharedRef<FInternetAddr> IpAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();
					IpAddr->SetIp(FriendGameInfo.m_unGameIP);
					IpAddr->SetPort(FriendGameInfo.m_usGamePort);
					CurrentSessionSearch->QuerySettings.Set(FName(SEARCH_STEAM_HOSTIP), IpAddr->ToString(true), EOnlineComparisonOp::Equals);
#else //WITH_ENGINE
					ensure(false); //Not currently supported
#endif //WITH_ENGINE

					FOnlineAsyncTaskSteamFindServerForFriendSession* NewTask = new FOnlineAsyncTaskSteamFindServerForFriendSession(SteamSubsystem, CurrentSessionSearch, LocalUserNum, OnFindFriendSessionCompleteDelegates[LocalUserNum]);
					SteamSubsystem->QueueAsyncTask(NewTask);
					bSuccess = true;
				}
			}
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Ignoring friend search request while another search is pending"));
	}

	if (!bSuccess)
	{
		TArray<FOnlineSessionSearchResult> EmptyResult;
		TriggerOnFindFriendSessionCompleteDelegates(LocalUserNum, bSuccess, EmptyResult);
	}

	return bSuccess;
}

bool FOnlineSessionSteam::FindFriendSession(const FUniqueNetId& LocalUserId, const FUniqueNetId& Friend)
{
	// @todo: use proper LocalUserId
	return FindFriendSession(0, Friend);
}

bool FOnlineSessionSteam::FindFriendSession(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& FriendList)
{
	UE_LOG_ONLINE_SESSION(Display, TEXT("FOnlineSessionSteam::FindFriendSession(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& FriendList) - not implemented"));
	// @todo: use proper LocalUserId
	TArray<FOnlineSessionSearchResult> EmptyResult;
	TriggerOnFindFriendSessionCompleteDelegates(0, false, EmptyResult);
	return false;
}

bool FOnlineSessionSteam::PingSearchResults(const FOnlineSessionSearchResult& SearchResult)
{
	return false;
}

void FOnlineSessionSteam::CheckPendingSessionInvite()
{
	const TCHAR* CmdLine = FCommandLine::Get();
	FString CmdLineStr(CmdLine);

	const FString LobbyConnectCmd = TEXT("+connect_lobby");
	int32 ConnectIdx = CmdLineStr.Find(LobbyConnectCmd, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
	if (ConnectIdx != INDEX_NONE)
	{
		const TCHAR* Str = CmdLine + ConnectIdx + LobbyConnectCmd.Len();
		FString LobbyIdStr = FParse::Token(Str, 0);
		int64 LobbyId = FCString::Strtoui64(*LobbyIdStr, nullptr, 10);
		if (LobbyId > 0)
		{
			PendingInvite.PendingInviteType = ESteamSession::LobbySession;
			PendingInvite.LobbyId = FUniqueNetIdSteam::Create(LobbyId);
		}
	}
	else
	{
		const FString ServerConnectCmd = TEXT("+connect");
		ConnectIdx = CmdLineStr.Find(ServerConnectCmd, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		if (ConnectIdx != INDEX_NONE)
		{
			const TCHAR* Str = CmdLine + ConnectIdx + ServerConnectCmd.Len();
			FString ServerIpAddrStr = FParse::Token(Str, 0);
			if (!ServerIpAddrStr.IsEmpty())
			{
				PendingInvite.PendingInviteType = ESteamSession::AdvertisedSessionClient;
				PendingInvite.ServerIp = FString::Printf(TEXT("-SteamConnectIP=%s"), *ServerIpAddrStr);
			}
		}
	}
}

bool FOnlineSessionSteam::SendSessionInviteToFriend(int32 LocalUserNum, FName SessionName, const FUniqueNetId& Friend)
{
	TArray<FUniqueNetIdRef> Friends;
	Friends.Add(Friend.AsShared());
	return SendSessionInviteToFriends(LocalUserNum, SessionName, Friends);
}

bool FOnlineSessionSteam::SendSessionInviteToFriend(const FUniqueNetId& LocalUserId, FName SessionName, const FUniqueNetId& Friend)
{
	// @todo: use proper LocalUserId
	return SendSessionInviteToFriend(0, SessionName, Friend);
}

bool FOnlineSessionSteam::SendSessionInviteToFriends(int32 LocalUserNum, FName SessionName, const TArray< FUniqueNetIdRef >& Friends)
{
	bool bSuccess = false;

	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session && Session->SessionInfo.IsValid())
	{
		FOnlineSessionInfoSteam* SessionInfo = (FOnlineSessionInfoSteam*)(Session->SessionInfo.Get());
		if (SessionInfo->SessionType == ESteamSession::LobbySession && SessionInfo->SessionId->IsValid())
		{
			for (int32 FriendIdx=0; FriendIdx < Friends.Num(); FriendIdx++)
			{
				const FUniqueNetIdSteam& FriendId = FUniqueNetIdSteam::Cast(*Friends[FriendIdx]);

				// Outside game accept -> +connect_lobby <64-bit lobby id> on client commandline
				// Inside game accept -> GameLobbyJoinRequested_t callback on client
				if (SteamMatchmaking()->InviteUserToLobby(*SessionInfo->SessionId, FriendId))
				{
					bSuccess = true;
				}
				else
				{
					UE_LOG_ONLINE_SESSION(Warning, TEXT("Error inviting %s to session %s, not connected to Steam"), *FriendId.ToDebugString(), *SessionName.ToString());
				}
			}
		}
		else if (SessionInfo->SessionType == ESteamSession::AdvertisedSessionHost || SessionInfo->SessionType == ESteamSession::AdvertisedSessionClient)
		{
			// Create the connection string
			FString ConnectionURL = GetSteamConnectionString(SessionName); 

			for (int32 FriendIdx=0; FriendIdx < Friends.Num(); FriendIdx++)
			{
				FUniqueNetIdSteam& FriendId = (FUniqueNetIdSteam&)(Friends[FriendIdx].Get());

				// Outside game accept -> the ConnectionURL gets added on client commandline
				// Inside game accept -> GameRichPresenceJoinRequested_t callback on client
				if (SteamFriends()->InviteUserToGame(FriendId, TCHAR_TO_UTF8(*ConnectionURL)))
				{
					UE_LOG_ONLINE_SESSION(Verbose, TEXT("Inviting %s to session %s with %s"), *FriendId.ToDebugString(), *SessionName.ToString(), *ConnectionURL);
					bSuccess = true;
				}
				else
				{
					UE_LOG_ONLINE_SESSION(Warning, TEXT("Error inviting %s to session %s"), *FriendId.ToDebugString(), *SessionName.ToString());
				}
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Invalid session info for invite %s"), *SessionName.ToString());
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Missing or invalid session %s for invite request"), *SessionName.ToString());
	}

	return bSuccess;
}

bool FOnlineSessionSteam::SendSessionInviteToFriends(const FUniqueNetId& LocalUserId, FName SessionName, const TArray< FUniqueNetIdRef >& Friends)
{
	// @todo: use proper LocalUserId
	return SendSessionInviteToFriends(0, SessionName, Friends);
}

FString FOnlineSessionSteam::GetSteamConnectionString(FName SessionName)
{
	FString ConnectionString;

	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session && Session->SessionInfo.IsValid())
	{
		FOnlineSessionInfoSteam* SessionInfo = (FOnlineSessionInfoSteam*)(Session->SessionInfo.Get());
		if (SessionInfo->SessionType == ESteamSession::AdvertisedSessionHost || SessionInfo->SessionType == ESteamSession::AdvertisedSessionClient)
		{
			ConnectionString = FString::Printf(TEXT("-SteamConnectIP=%s"), *SessionInfo->HostAddr->ToString(true));
		}
	}

	return ConnectionString;
}

/** Get a resolved connection string from a session info */
static bool GetConnectStringFromSessionInfo(TSharedPtr<FOnlineSessionInfoSteam>& SessionInfo, FString& ConnectInfo, int32 PortOverride=0)
{
	bool bSuccess = false;

	if (SessionInfo.IsValid())
	{
		bool bP2PDataValid = (SessionInfo->SteamP2PAddr.IsValid() && SessionInfo->SteamP2PAddr->IsValid());
		bool bHostDataValid = (SessionInfo->HostAddr.IsValid() && SessionInfo->HostAddr->IsValid());

		// If we have host data, attempt to use it.
		if (bHostDataValid && SessionInfo->ConnectionMethod == FSteamConnectionMethod::Direct)
		{
			UE_LOG_ONLINE_SESSION(Log, TEXT("Using Host Data for Connection Serialization"));

			int32 HostPort = SessionInfo->HostAddr->GetPort();
			if (PortOverride > 0)
			{
				HostPort = PortOverride;
			}

			ConnectInfo = FString::Printf(TEXT("%s:%d"), *SessionInfo->HostAddr->ToString(false), HostPort);
			bSuccess = true;
		}
		else if (bP2PDataValid)
		{
			UE_LOG_ONLINE_SESSION(Log, TEXT("Using P2P Data for Connection Serialization"));

			int32 SteamPort = SessionInfo->SteamP2PAddr->GetPort();
			if (PortOverride > 0)
			{
				SteamPort = PortOverride;
			}

			ConnectInfo = FString::Printf(STEAM_URL_PREFIX TEXT("%s:%d"), *SessionInfo->SteamP2PAddr->ToString(false), SteamPort);
			bSuccess = true;
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Steam could not resolve session info! ValidP2P[%d] ValidHost[%d] ConnectionMethod[%s]"), 
				bP2PDataValid, bHostDataValid, *LexToString(SessionInfo->ConnectionMethod));
			return false;
		}
	}

	return bSuccess;
}

bool FOnlineSessionSteam::GetResolvedConnectString(FName SessionName, FString& ConnectInfo, FName PortType)
{
	bool bSuccess = false;
	// Find the session
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session != nullptr)
	{
		TSharedPtr<FOnlineSessionInfoSteam> SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoSteam>(Session->SessionInfo);
		if (PortType == NAME_BeaconPort)
		{
			int32 BeaconListenPort = GetBeaconPortFromSessionSettings(Session->SessionSettings);
			bSuccess = GetConnectStringFromSessionInfo(SessionInfo, ConnectInfo, BeaconListenPort);
		}
		else if (PortType == NAME_GamePort)
		{
			bSuccess = GetConnectStringFromSessionInfo(SessionInfo, ConnectInfo);
		}

		if (!bSuccess)
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Invalid session info for session %s in GetResolvedConnectString()"), *SessionName.ToString());
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning,
			TEXT("Unknown session name (%s) specified to GetResolvedConnectString()"),
			*SessionName.ToString());
	}

	return bSuccess;
}

bool FOnlineSessionSteam::GetResolvedConnectString(const FOnlineSessionSearchResult& SearchResult, const FName PortType, FString& ConnectInfo)
{
	bool bSuccess = false;
	if (SearchResult.Session.SessionInfo.IsValid())
	{
		TSharedPtr<FOnlineSessionInfoSteam> SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoSteam>(SearchResult.Session.SessionInfo);

		if (PortType == NAME_BeaconPort)
		{
			int32 BeaconListenPort = GetBeaconPortFromSessionSettings(SearchResult.Session.SessionSettings);
			bSuccess = GetConnectStringFromSessionInfo(SessionInfo, ConnectInfo, BeaconListenPort);
		}
		else if (PortType == NAME_GamePort)
		{
			bSuccess = GetConnectStringFromSessionInfo(SessionInfo, ConnectInfo);
		}
	}

	if (!bSuccess || ConnectInfo.IsEmpty())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Invalid session info in search result to GetResolvedConnectString()"));
	}

	return bSuccess;
}

FString FOnlineSessionSteam::GetCustomDedicatedServerName() const
{
	FString ServerName;

	if (!FParse::Value(FCommandLine::Get(), TEXT("-SteamServerName="), ServerName))
	{
		GConfig->GetString(TEXT("OnlineSubsystemSteam"), TEXT("SteamServerName"), ServerName, GEngineIni);
	}

	if (ServerName.Len() >= k_cbMaxGameServerName)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("SteamServerName overflows the maximum amount of characters %d allowed, truncating."), k_cbMaxGameServerName);
		// Must have space for the null terminator
		ServerName.LeftInline(k_cbMaxGameServerName - 1);
	}

	return ServerName;
}

FUniqueNetIdPtr FOnlineSessionSteam::CreateSessionIdFromString(const FString& SessionIdStr)
{
	if (!SessionIdStr.IsEmpty())
	{
		return FUniqueNetIdSteam::Create(SessionIdStr);
	}
	return nullptr;
}

FOnlineSessionSettings* FOnlineSessionSteam::GetSessionSettings(FName SessionName) 
{
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		return &Session->SessionSettings;
	}
	return nullptr;
}

void FOnlineSessionSteam::RegisterVoice(const FUniqueNetId& PlayerId)
{
	if (!SteamSubsystem->IsDedicated())
	{
		if (PlayerId.IsValid())
		{
			IOnlineVoicePtr VoiceInt = SteamSubsystem->GetVoiceInterface();
			if (VoiceInt.IsValid())
			{
				if (!SteamSubsystem->IsLocalPlayer(PlayerId))
				{
					VoiceInt->RegisterRemoteTalker(PlayerId);
				}
				else
				{
					// This is a local player. In case their PlayerState came last during replication, reprocess muting
					VoiceInt->ProcessMuteChangeNotification();
				}
			}
		}
	}
}

void FOnlineSessionSteam::UnregisterVoice(const FUniqueNetId& PlayerId)
{
	if (!SteamSubsystem->IsDedicated())
	{
		IOnlineVoicePtr VoiceInt = SteamSubsystem->GetVoiceInterface();
		if (VoiceInt.IsValid())
		{
			if (PlayerId.IsValid() && !SteamSubsystem->IsLocalPlayer(PlayerId))
			{
				VoiceInt->UnregisterRemoteTalker(PlayerId);
			}
		}
	}
}

bool FOnlineSessionSteam::RegisterPlayer(FName SessionName, const FUniqueNetId& PlayerId, bool bWasInvited)
{
	TArray< FUniqueNetIdRef > Players;
	Players.Add(PlayerId.AsShared());
	return RegisterPlayers(SessionName, Players, bWasInvited);
}

bool FOnlineSessionSteam::RegisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players, bool bWasInvited)
{
	bool bSuccess = false;
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		if (Session->SessionInfo.IsValid())
		{
			FOnlineSessionInfoSteam* SessionInfo = (FOnlineSessionInfoSteam*)(Session->SessionInfo.Get());

			ISteamFriends* SteamFriendsPtr = SteamFriends();
			for (int32 PlayerIdx=0; PlayerIdx < Players.Num(); PlayerIdx++)
			{
				const FUniqueNetIdRef& PlayerId = Players[PlayerIdx];
				const FUniqueNetIdSteam& SteamId = (const FUniqueNetIdSteam&)*PlayerId;

				FUniqueNetIdMatcher PlayerMatch(SteamId);
				if (Session->RegisteredPlayers.IndexOfByPredicate(PlayerMatch) == INDEX_NONE)
				{
					Session->RegisteredPlayers.Add(PlayerId);

					// Determine if this player is really remote or not
					if (!SteamSubsystem->IsLocalPlayer(SteamId))
					{
						if (SteamFriendsPtr)
						{
							SteamFriendsPtr->RequestUserInformation(SteamId, true);
						}
					}
				}
				else
				{
					UE_LOG_ONLINE_SESSION(Log, TEXT("Player %s already registered in session %s"), *Players[PlayerIdx]->ToDebugString(), *SessionName.ToString());
				}

				RegisterVoice(SteamId);
			}

			bSuccess = true;
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("No session info to join for session (%s)"), *SessionName.ToString());
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("No game present to join for session (%s)"), *SessionName.ToString());
	}

	TriggerOnRegisterPlayersCompleteDelegates(SessionName, Players, bSuccess);
	return bSuccess;
}

void FOnlineSessionSteam::RegisterLocalPlayers(FNamedOnlineSession* Session)
{
	if (!SteamSubsystem->IsDedicated())
	{
		IOnlineVoicePtr VoiceInt = SteamSubsystem->GetVoiceInterface();
		if (VoiceInt.IsValid())
		{
			for (int32 Index = 0; Index < MAX_LOCAL_PLAYERS; Index++)
			{
				// Register the local player as a local talker
				VoiceInt->RegisterLocalTalker(Index);
			}
		}
	}
}

bool FOnlineSessionSteam::UnregisterPlayer(FName SessionName, const FUniqueNetId& PlayerId)
{
	TArray< FUniqueNetIdRef > Players;
	Players.Add(PlayerId.AsShared());
	return UnregisterPlayers(SessionName, Players);
}

bool FOnlineSessionSteam::UnregisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players)
{
	bool bSuccess = false;

	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		if (Session->SessionInfo.IsValid())
		{
			FOnlineSessionInfoSteam* SessionInfo = (FOnlineSessionInfoSteam*)(Session->SessionInfo.Get());

			for (int32 PlayerIdx=0; PlayerIdx < Players.Num(); PlayerIdx++)
			{
				const FUniqueNetIdRef& PlayerId = Players[PlayerIdx];

				FUniqueNetIdMatcher PlayerMatch(*PlayerId);
				int32 RegistrantIndex = Session->RegisteredPlayers.IndexOfByPredicate(PlayerMatch);
				if (RegistrantIndex != INDEX_NONE)
				{
					Session->RegisteredPlayers.RemoveAtSwap(RegistrantIndex);
					UnregisterVoice(*PlayerId);
				}
				else
				{
					UE_LOG_ONLINE_SESSION(Warning, TEXT("Player %s is not part of session (%s)"), *PlayerId->ToDebugString(), *SessionName.ToString());
				}
			}

			bSuccess = true;
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("No session info to leave for session (%s)"), *SessionName.ToString());
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("No game present to leave for session (%s)"), *SessionName.ToString());
	}

	TriggerOnUnregisterPlayersCompleteDelegates(SessionName, Players, bSuccess);
	return bSuccess;
}

void FOnlineSessionSteam::Tick(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_Session_Interface);
	TickLanTasks(DeltaTime);
	TickPendingInvites(DeltaTime);
}

void FOnlineSessionSteam::TickLanTasks(float DeltaTime)
{
	if (LANSession != nullptr && LANSession->GetBeaconState() > ELanBeaconState::NotUsingLanBeacon)
	{
		LANSession->Tick(DeltaTime);
	}
}

void FOnlineSessionSteam::TickPendingInvites(float DeltaTime)
{
	if (PendingInvite.PendingInviteType != ESteamSession::None)
	{
		if (OnSessionUserInviteAcceptedDelegates.IsBound())
		{
			FOnlineAsyncItem* NewEvent = nullptr;
			if (PendingInvite.PendingInviteType == ESteamSession::LobbySession)
			{
				NewEvent = new FOnlineAsyncEventSteamLobbyInviteAccepted(SteamSubsystem, *FUniqueNetIdSteam::EmptyId(), *PendingInvite.LobbyId);
			}
			else
			{
				NewEvent = new FOnlineAsyncEventSteamInviteAccepted(SteamSubsystem, *FUniqueNetIdSteam::EmptyId(), PendingInvite.ServerIp);
			}

			if (NewEvent)
			{
				UE_LOG_ONLINE_SESSION(Verbose, TEXT("%s"), *NewEvent->ToString());
				SteamSubsystem->QueueAsyncOutgoingItem(NewEvent);
			}

			// Clear the invite
			PendingInvite.PendingInviteType = ESteamSession::None;
		}
	}
}

void FOnlineSessionSteam::AppendSessionToPacket(FNboSerializeToBufferSteam& Packet, FOnlineSession* Session)
{
	/** Owner of the session */
	((FNboSerializeToBuffer&)Packet) << StaticCastSharedPtr<const FUniqueNetIdSteam>(Session->OwningUserId)->UniqueNetId
		<< Session->OwningUserName
		<< Session->NumOpenPrivateConnections
		<< Session->NumOpenPublicConnections;

	// Write host info (host addr, session id, and key)
	Packet << *StaticCastSharedPtr<FOnlineSessionInfoSteam>(Session->SessionInfo);

	// Now append per game settings
	AppendSessionSettingsToPacket(Packet, &Session->SessionSettings);
}

void FOnlineSessionSteam::AppendSessionSettingsToPacket(FNboSerializeToBufferSteam& Packet, FOnlineSessionSettings* SessionSettings)
{
#if DEBUG_LAN_BEACON
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("Sending session settings to client"));
#endif 

	// Members of the session settings class
	((FNboSerializeToBuffer&)Packet) << SessionSettings->NumPublicConnections
		<< SessionSettings->NumPrivateConnections
		<< (uint8)SessionSettings->bShouldAdvertise
		<< (uint8)SessionSettings->bIsLANMatch
		<< (uint8)SessionSettings->bIsDedicated
		<< (uint8)SessionSettings->bUsesStats
		<< (uint8)SessionSettings->bAllowJoinInProgress
		<< (uint8)SessionSettings->bAllowInvites
		<< (uint8)SessionSettings->bUsesPresence
		<< (uint8)SessionSettings->bUseLobbiesIfAvailable
		<< (uint8)SessionSettings->bAllowJoinViaPresence
		<< (uint8)SessionSettings->bAllowJoinViaPresenceFriendsOnly
		<< (uint8)SessionSettings->bAntiCheatProtected
	    << SessionSettings->BuildUniqueId;

	// First count number of advertised keys
	int32 NumAdvertisedProperties = 0;
	for (FSessionSettings::TConstIterator It(SessionSettings->Settings); It; ++It)
	{	
		const FOnlineSessionSetting& Setting = It.Value();
		if (Setting.AdvertisementType >= EOnlineDataAdvertisementType::ViaOnlineService)
		{
			NumAdvertisedProperties++;
		}
	}

	// Add count of advertised keys and the data
	((FNboSerializeToBuffer&)Packet) << (int32)NumAdvertisedProperties;
	for (FSessionSettings::TConstIterator It(SessionSettings->Settings); It; ++It)
	{
		const FOnlineSessionSetting& Setting = It.Value();
		if (Setting.AdvertisementType >= EOnlineDataAdvertisementType::ViaOnlineService)
		{
			((FNboSerializeToBuffer&)Packet) << It.Key();
			Packet << Setting;
#if DEBUG_LAN_BEACON
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("%s"), *Setting.ToString());
#endif
		}
	}
}

void FOnlineSessionSteam::OnValidQueryPacketReceived(uint8* PacketData, int32 PacketLength, uint64 ClientNonce)
{
	// Iterate through all registered sessions and respond for each LAN match
	FScopeLock ScopeLock(&SessionLock);
	for (int32 SessionIndex = 0; SessionIndex < Sessions.Num(); SessionIndex++)
	{
		FNamedOnlineSession& Session = Sessions[SessionIndex];

		const FOnlineSessionSettings& Settings = Session.SessionSettings;

		const bool bIsMatchInProgress = Session.SessionState == EOnlineSessionState::InProgress;

		const bool bIsMatchJoinable = Settings.bIsLANMatch &&
			(!bIsMatchInProgress || Settings.bAllowJoinInProgress) &&
			Settings.NumPublicConnections > 0;

		// Don't respond to query if the session is not a joinable LAN match.
		if (bIsMatchJoinable)
		{
			FNboSerializeToBufferSteam Packet(LAN_BEACON_MAX_PACKET_SIZE);
			// Create the basic header before appending additional information
			LANSession->CreateHostResponsePacket(Packet, ClientNonce);
			
			// Add all the session details
			AppendSessionToPacket(Packet, &Session);

			// Broadcast this response so the client can see us
			LANSession->BroadcastPacket(Packet, Packet.GetByteCount());
		}
	}
}

void FOnlineSessionSteam::ReadSessionFromPacket(FNboSerializeFromBufferSteam& Packet, FOnlineSession* Session)
{
#if DEBUG_LAN_BEACON
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("Reading session information from server"));
#endif

	uint64 OwningUserId;
	Packet >> OwningUserId
		>> Session->OwningUserName
		>> Session->NumOpenPrivateConnections
		>> Session->NumOpenPublicConnections;

	Session->OwningUserId = FUniqueNetIdSteam::Create(OwningUserId);

	// Allocate and read the connection data
	FOnlineSessionInfoSteam* SteamSessionInfo = new FOnlineSessionInfoSteam(ESteamSession::LANSession);
	SteamSessionInfo->HostAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();
	Packet >> *SteamSessionInfo;
	Session->SessionInfo = MakeShareable(SteamSessionInfo); 

	// Read any per object data using the server object
	ReadSettingsFromPacket(Packet, Session->SessionSettings);
}

void FOnlineSessionSteam::ReadSettingsFromPacket(FNboSerializeFromBufferSteam& Packet, FOnlineSessionSettings& SessionSettings)
{
#if DEBUG_LAN_BEACON
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("Reading game settings from server"));
#endif

	// Clear out any old settings
	SessionSettings.Settings.Empty();

	// Members of the session settings class
	Packet >> SessionSettings.NumPublicConnections
		>> SessionSettings.NumPrivateConnections;
	uint8 Read = 0;
	// Read all the bools as bytes
	Packet >> Read;
	SessionSettings.bShouldAdvertise = !!Read;
	Packet >> Read;
	SessionSettings.bIsLANMatch = !!Read;
	Packet >> Read;
	SessionSettings.bIsDedicated = !!Read;
	Packet >> Read;
	SessionSettings.bUsesStats = !!Read;
	Packet >> Read;
	SessionSettings.bAllowJoinInProgress = !!Read;
	Packet >> Read;
	SessionSettings.bAllowInvites = !!Read;
	Packet >> Read;
	SessionSettings.bUsesPresence = !!Read;
	Packet >> Read;
	SessionSettings.bUseLobbiesIfAvailable = !!Read;
	Packet >> Read;
	SessionSettings.bAllowJoinViaPresence = !!Read;
	Packet >> Read;
	SessionSettings.bAllowJoinViaPresenceFriendsOnly = !!Read;
	Packet >> Read;
	SessionSettings.bAntiCheatProtected = !!Read;

	// BuildId
	Packet >> SessionSettings.BuildUniqueId;

	// Now read the contexts and properties from the settings class
	int32 NumAdvertisedProperties = 0;
	// First, read the number of advertised properties involved, so we can presize the array
	Packet >> NumAdvertisedProperties;
	if (Packet.HasOverflow() == false)
	{
		FName Key;
		// Now read each context individually
		for (int32 Index = 0;
			Index < NumAdvertisedProperties && Packet.HasOverflow() == false;
			Index++)
		{
			FOnlineSessionSetting Setting;
			Packet >> Key;
			Packet >> Setting;
			SessionSettings.Set(Key, Setting);

#if DEBUG_LAN_BEACON
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("%s"), *Setting->ToString());
#endif
		}
	}
	
	// If there was an overflow, treat the string settings/properties as broken
	if (Packet.HasOverflow())
	{
		SessionSettings.Settings.Empty();
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("Packet overflow detected in ReadGameSettingsFromPacket()"));
	}
}

void FOnlineSessionSteam::OnValidResponsePacketReceived(uint8* PacketData, int32 PacketLength)
{
	// Create an object that we'll copy the data to
	FOnlineSessionSettings NewServer;
	if (CurrentSessionSearch.IsValid())
	{
		// Add space in the search results array
		FOnlineSessionSearchResult* NewResult = new (CurrentSessionSearch->SearchResults) FOnlineSessionSearchResult();
		FOnlineSession* NewSession = &NewResult->Session;

		// Prepare to read data from the packet
		FNboSerializeFromBufferSteam Packet(PacketData, PacketLength);
		
		ReadSessionFromPacket(Packet, NewSession);

		// NOTE: we don't notify until the timeout happens
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to create new online game settings object"));
	}
}

void FOnlineSessionSteam::OnLANSearchTimeout()
{
	// See if there were any sessions that were marked as hosting before the search started
	bool bWasHosting = false;

	{
		FScopeLock ScopeLock(&SessionLock);
		for (int32 SessionIdx=0; SessionIdx < Sessions.Num(); SessionIdx++)
		{
			FNamedOnlineSession& Session = Sessions[SessionIdx];
			if (Session.SessionSettings.bShouldAdvertise &&
				Session.SessionSettings.bIsLANMatch &&
				SteamSubsystem->IsServer())
			{
				bWasHosting = true;
				break;
			}
		}
	}

	if (bWasHosting)
	{
		FOnValidQueryPacketDelegate QueryPacketDelegate = FOnValidQueryPacketDelegate::CreateRaw(this, &FOnlineSessionSteam::OnValidQueryPacketReceived);
		// Maintain lan beacon if there was a session that was marked as hosting
		if (LANSession->Host(QueryPacketDelegate))
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to restart hosted LAN session after search completion"));
		}
	}
	else
	{
		// Stop future timeouts since we aren't searching any more
		LANSession->StopLANSession();
	}

	if (CurrentSessionSearch.IsValid())
	{
		if (CurrentSessionSearch->SearchResults.Num() > 0)
		{
			// Allow game code to sort the servers
			CurrentSessionSearch->SortSearchResults();
		}
		CurrentSessionSearch->SearchState = EOnlineAsyncTaskState::Done;

		CurrentSessionSearch = nullptr;
	}

	// Trigger the delegate as complete
	TriggerOnFindSessionsCompleteDelegates(true);
}

void FOnlineSessionSteam::SyncLobbies()
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("Member of %d lobbies"), JoinedLobbyList.Num());
	TArray<FUniqueNetIdSteamRef> LobbiesToRemove = JoinedLobbyList;

	{
		FScopeLock ScopeLock(&SessionLock);
		for (int32 SessionIdx=0; SessionIdx < Sessions.Num(); SessionIdx++)
		{
			const FNamedOnlineSession& Session = Sessions[SessionIdx];
			FOnlineSessionInfoSteam* SessionInfo = (FOnlineSessionInfoSteam*)(Session.SessionInfo.Get());
			if (SessionInfo->SessionType == ESteamSession::LobbySession && SessionInfo->SessionId->IsValid())
			{
				LobbiesToRemove.RemoveSingleSwap(SessionInfo->SessionId);
			}
		}
	}

	for (int32 LobbyIdx=0; LobbyIdx < LobbiesToRemove.Num(); LobbyIdx++)
	{
		const FUniqueNetIdSteam& LobbyId = *LobbiesToRemove[LobbyIdx];
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("Lobby %s out of sync, removing..."), *LobbyId.ToDebugString());
		FOnlineAsyncTaskSteamLeaveLobby* NewTask = new FOnlineAsyncTaskSteamLeaveLobby(SteamSubsystem, TEXT("OUTOFSYNC"), LobbyId);
		SteamSubsystem->QueueAsyncTask(NewTask);
	}
}

int32 FOnlineSessionSteam::GetNumSessions()
{
	FScopeLock ScopeLock(&SessionLock);
	return Sessions.Num();
}

void FOnlineSessionSteam::DumpSessionState()
{
	FScopeLock ScopeLock(&SessionLock);

	UE_LOG_ONLINE_SESSION(Verbose, TEXT("Member of %d lobbies"), JoinedLobbyList.Num());
	TArray<FUniqueNetIdSteamRef> OutOfSyncLobbies = JoinedLobbyList;
	for (int32 SessionIdx=0; SessionIdx < Sessions.Num(); SessionIdx++)
	{
		const FNamedOnlineSession& Session = Sessions[SessionIdx];
		FOnlineSessionInfoSteam* SessionInfo = (FOnlineSessionInfoSteam*)(Session.SessionInfo.Get());
		if (SessionInfo->SessionType == ESteamSession::LobbySession && SessionInfo->SessionId->IsValid())
		{
			OutOfSyncLobbies.RemoveSingleSwap(SessionInfo->SessionId);
		}
	}

	if (OutOfSyncLobbies.Num() > 0)
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("Out of sync lobbies: %d"), OutOfSyncLobbies.Num());
		for (int32 LobbyIdx=0; LobbyIdx < OutOfSyncLobbies.Num(); LobbyIdx++)
		{
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("%s"), *OutOfSyncLobbies[LobbyIdx]->ToDebugString());
		}
	}

	for (int32 SessionIdx=0; SessionIdx < Sessions.Num(); SessionIdx++)
	{
		DumpNamedSession(&Sessions[SessionIdx]);
	}
}

void FOnlineSessionSteam::RegisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnRegisterLocalPlayerCompleteDelegate& Delegate)
{
	Delegate.ExecuteIfBound(PlayerId, EOnJoinSessionCompleteResult::Success);
}

void FOnlineSessionSteam::UnregisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnUnregisterLocalPlayerCompleteDelegate& Delegate)
{
	Delegate.ExecuteIfBound(PlayerId, true);
}

/** Implementation of the ConnectionMethod converters */
FString LexToString(const FSteamConnectionMethod Method)
{
	switch (Method)
	{
	default:
	case FSteamConnectionMethod::None:
		return TEXT("None");
	case FSteamConnectionMethod::Direct:
		return TEXT("Direct");
	case FSteamConnectionMethod::P2P:
		return TEXT("P2P");
	case FSteamConnectionMethod::PartnerHosted:
		return TEXT("PartnerHosted");
	}
}

FSteamConnectionMethod ToConnectionMethod(const FString& InString)
{
	if (InString == TEXT("Direct"))
	{
		return FSteamConnectionMethod::Direct;
	}
	else if (InString == TEXT("P2P"))
	{
		return FSteamConnectionMethod::P2P;
	}
	else if (InString == TEXT("PartnerHosted"))
	{
		return FSteamConnectionMethod::PartnerHosted;
	}
	else
	{
		return FSteamConnectionMethod::None;
	}
}
