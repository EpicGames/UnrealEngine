// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineSessionDelegates.h"
#include "OnlineSubsystemPackage.h"
#include "OnlineSubsystemSteamTypes.h"
#include "OnlineSessionSettings.h"
#include "Interfaces/OnlineSessionInterface.h"

/** Async Task timeout value */
#define ASYNC_TASK_TIMEOUT 15.0f

/** Structure to hold key value pairs (as FStrings) for Steam */
typedef FOnlineKeyValuePairs<FString, FString> FSteamSessionKeyValuePairs;

/**
 * Interface definition for the online services session services 
 * Session services are defined as anything related managing a session 
 * and its state within a platform service
 */
class FOnlineSessionSteam : public IOnlineSession
{
private:

	/** Reference to the main Steam subsystem */
	class FOnlineSubsystemSteam* SteamSubsystem;

	/** Instance of a LAN session for hosting/client searches */
	class FLANSession* LANSession;

	/** Hidden on purpose */
	FOnlineSessionSteam() :
		SteamSubsystem(NULL),
		LANSession(NULL),
		bSteamworksGameServerConnected(false),
		GameServerSteamId(NULL),
		bPolicyResponseReceived(false),
		CurrentSessionSearch(NULL)
	{}

	/**
	 * Ticks any lan beacon background tasks
	 *
	 * @param DeltaTime the time since the last tick
	 */
	void TickLanTasks(float DeltaTime);

	/**
	 * Tick invites captured from the command line
	 * Waits for a delegate to be listening before triggering
	 */
	void TickPendingInvites(float DeltaTime);

	/**
	 *	Create a lobby session, advertised on the Steam backend
	 * 
	 * @param HostingPlayerNum local index of the user initiating the request
     * @param Session newly allocated session to create
	 * 
	 * @return ONLINE_SUCCESS if successful, an error code otherwise
	 */
	uint32 CreateLobbySession(int32 HostingPlayerNum, class FNamedOnlineSession* Session);

	/**
	 *	Create a game server session, advertised on the Steam backend
	 * 
	 * @param HostingPlayerNum local index of the user initiating the request
     * @param Session newly allocated session to create
	 * 
	 * @return ONLINE_SUCCESS if successful, an error code otherwise
	 */
	uint32 CreateInternetSession(int32 HostingPlayerNum, class FNamedOnlineSession* Session);

	/**
	 *	Join a lobby session, advertised on the Steam backend
	 * 
	 * @param PlayerNum local index of the user initiating the request
	 * @param Session newly allocated session with join information
	 * @param SearchSession the desired session to join
	 * 
	 * @return ONLINE_SUCCESS if successful, an error code otherwise
	 */
	uint32 JoinLobbySession(int32 PlayerNum, class FNamedOnlineSession* Session, const FOnlineSession* SearchSession);

	/**
	 *	Join a game server session, advertised on the Steam backend
	 * 
	 * @param PlayerNum local index of the user initiating the request
	 * @param Session newly allocated session with join information
	 * @param SearchSession the desired session to join
	 * 
	 * @return ONLINE_SUCCESS if successful, an error code otherwise
	 */
	uint32 JoinInternetSession(int32 PlayerNum, FNamedOnlineSession* Session, const FOnlineSession* SearchSession);

	/**
	 *	End an internet session, advertised on the Steam backend
	 * NOTE: Lobby sessions remain active until DestroyOnlineSession
	 * 
     * @param Session session to end
	 * 
	 * @return ONLINE_SUCCESS if successful, an error code otherwise
	 */
	uint32 EndInternetSession(class FNamedOnlineSession* Session);

	/**
	 *	Destroy a lobby session, advertised on the Steam backend
	 * 
	 * @param Session session to destroy
	 * 
	 * @return ONLINE_SUCCESS if successful, an error code otherwise
	 */
	uint32 DestroyLobbySession(class FNamedOnlineSession* Session, const FOnDestroySessionCompleteDelegate& CompletionDelegate);

	/**
	 *	Destroy an internet session, advertised on the Steam backend
	 * 
	 * @param Session session to destroy
	 * 
	 * @return ONLINE_SUCCESS if successful, an error code otherwise
	 */
	uint32 DestroyInternetSession(class FNamedOnlineSession* Session, const FOnDestroySessionCompleteDelegate& CompletionDelegate);

	/**
	 *	Create a local LAN session, managed by a beacon on the host
	 * 
	 * @param HostingPlayerNum local index of the user initiating the request
     * @param Session newly allocated session to create
	 * 
	 * @return ONLINE_SUCCESS if successful, an error code otherwise
	 */
	uint32 CreateLANSession(int32 HostingPlayerNum, class FNamedOnlineSession* Session);

	/**
	 *	Join a LAN session
	 * 
	 * @param PlayerNum local index of the user initiating the request
	 * @param Session newly allocated session with join information
	 * @param SearchSession the desired session to join
	 * 
	 * @return ONLINE_SUCCESS if successful, an error code otherwise
	 */
	uint32 JoinLANSession(int32 PlayerNum, class FNamedOnlineSession* Session, const class FOnlineSession* SearchSession);

	/**
	 * Builds a Steamworks query and submits it to the Steamworks backend
	 *
	 * @return an Error/success code
	 */
	uint32 FindInternetSession(const TSharedRef<FOnlineSessionSearch>& SearchSettings);

	/**
	 * Builds a LAN search query and broadcasts it
	 *
	 * @return ONLINE_SUCCESS if successful, an error code otherwise
	 */
	uint32 FindLANSession(const TSharedRef<FOnlineSessionSearch>& SearchSettings);

	/**
	 * Adds the game session data to the packet that is sent by the host
	 * in response to a server query
	 *
	 * @param Packet the writer object that will encode the data
	 * @param Session the session to add to the packet
	 */
	void AppendSessionToPacket(class FNboSerializeToBufferSteam& Packet, class FOnlineSession* Session);

	/**
	 * Adds the game settings data to the packet that is sent by the host
	 * in response to a server query
	 *
	 * @param Packet the writer object that will encode the data
	 * @param SessionSettings the session settings to add to the packet
	 */
	void AppendSessionSettingsToPacket(class FNboSerializeToBufferSteam& Packet, FOnlineSessionSettings* SessionSettings);

	/**
	 * Reads the settings data from the packet and applies it to the
	 * specified object
	 *
	 * @param Packet the reader object that will read the data
	 * @param SessionSettings the session settings to copy the data to
	 */
	void ReadSessionFromPacket(class FNboSerializeFromBufferSteam& Packet, class FOnlineSession* Session);

	/**
	 * Reads the settings data from the packet and applies it to the
	 * specified object
	 *
	 * @param Packet the reader object that will read the data
	 * @param SessionSettings the session settings to copy the data to
	 */
	void ReadSettingsFromPacket(class FNboSerializeFromBufferSteam& Packet, FOnlineSessionSettings& SessionSettings);

	/**
	 * Delegate triggered when the LAN beacon has detected a valid client request has been received
	 *
	 * @param PacketData packet data sent by the requesting client with header information removed
	 * @param PacketLength length of the packet not including header size
	 * @param ClientNonce the nonce returned by the client to return with the server packet
	 */
	void OnValidQueryPacketReceived(uint8* PacketData, int32 PacketLength, uint64 ClientNonce);

	/**
	 * Delegate triggered when the LAN beacon has detected a valid host response to a client request has been received
	 *
	 * @param PacketData packet data sent by the requesting client with header information removed
	 * @param PacketLength length of the packet not including header size
	 */
	void OnValidResponsePacketReceived(uint8* PacketData, int32 PacketLength);

	/**
	 * Delegate triggered when the LAN beacon has finished searching (some time after last received host packet)
	 */
	void OnLANSearchTimeout();

	/**
	 * Registers and updates voice data for the given player id
	 *
	 * @param PlayerId player to register with the voice subsystem
	 */
	void RegisterVoice(const FUniqueNetId& PlayerId);

	/**
	 * Unregisters a given player id from the voice subsystem
	 *
	 * @param PlayerId player to unregister with the voice subsystem
	 */
	void UnregisterVoice(const FUniqueNetId& PlayerId);

PACKAGE_SCOPE:

	/** Critical sections for thread safe operation of session lists */
	mutable FCriticalSection SessionLock;

	/** Current session settings */
	TArray<FNamedOnlineSession> Sessions;

	/** Whether or not the Steam game server API is fully logged in and connected (GameServerSteamId is valid) */
	bool bSteamworksGameServerConnected;

	/** CSteamId assigned on game server login (only valid if bSteamworksGameServerConnected is true) */
	FUniqueNetIdSteamPtr GameServerSteamId;

	/** Has the GSPolicyResponse callback triggered */
	bool bPolicyResponseReceived;

	/** Current search object */
	TSharedPtr<FOnlineSessionSearch> CurrentSessionSearch;

	/** Any invite/join from the command line */
	struct FPendingInviteData
	{
		/** What kind of invite is this */
		ESteamSession::Type PendingInviteType;
		/** Lobby invite information */
		FUniqueNetIdSteamRef LobbyId;
		/** Server invite information */
		FString ServerIp;

		FPendingInviteData() :
			PendingInviteType(ESteamSession::None),
			LobbyId(FUniqueNetIdSteam::EmptyId())
		{
		}
	};

	/** Contains information about a join/invite parsed from the commandline */
	FPendingInviteData PendingInvite;

	/** List of lobby data that is available for parsing (READ/WRITE game thread READONLY online thread) */
	TArray<FUniqueNetIdSteamRef> PendingSearchLobbyIds;

	/** Critical sections for thread safe operation of the lobby lists */
	FCriticalSection JoinedLobbyLock;

	/** List of lobbies this client is a member of */
	TArray<FUniqueNetIdSteamRef> JoinedLobbyList;

	FOnlineSessionSteam(class FOnlineSubsystemSteam* InSubsystem) :
		SteamSubsystem(InSubsystem),
		LANSession(NULL),
		bSteamworksGameServerConnected(false),
		GameServerSteamId(NULL),
		bPolicyResponseReceived(false),
		CurrentSessionSearch(NULL)
	{}

	/**
	 * Session tick for various background tasks
	 */
	void Tick(float DeltaTime);

	/**
	 * Adds a new named session to the list (new session)
	 *
	 * @param SessionName the name to search for
	 * @param GameSettings the game settings to add
	 *
	 * @return a pointer to the struct that was added
	 */
	class FNamedOnlineSession* AddNamedSession(FName SessionName, const FOnlineSessionSettings& SessionSettings) override
	{
		FScopeLock ScopeLock(&SessionLock);
		return &Sessions.Emplace_GetRef(SessionName, SessionSettings);
	}

	/**
	 * Adds a new named session to the list (from existing session data)
	 *
	 * @param SessionName the name to search for
	 * @param GameSettings the game settings to add
	 *
	 * @return a pointer to the struct that was added
	 */
	class FNamedOnlineSession* AddNamedSession(FName SessionName, const FOnlineSession& Session) override
	{
		FScopeLock ScopeLock(&SessionLock);
		return &Sessions.Emplace_GetRef(SessionName, Session);
	}

	/**
	 * Searches the named session array for the specified session
	 *
	 * @param LobbyId the lobby id to search for
	 *
	 * @return pointer to the struct if found, NULL otherwise
	 */
	inline FNamedOnlineSession* GetNamedSessionFromLobbyId(const FUniqueNetIdSteam& LobbyId)
	{
		FScopeLock ScopeLock(&SessionLock);
		for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
		{
			FNamedOnlineSession& Session = Sessions[SearchIndex];
			if (Session.SessionInfo.IsValid())
			{
				FOnlineSessionInfoSteam* SessionInfo = (FOnlineSessionInfoSteam*)Session.SessionInfo.Get();
				if (SessionInfo->SessionType == ESteamSession::LobbySession && *SessionInfo->SessionId == LobbyId)
				{
					return &Sessions[SearchIndex];
				}
			}
		}
		return NULL;
	}

	/**
	 * Return the game server based session
	 * NOTE: Assumes there is at most one, non-lobby session
	 *
	 * @return pointer to the struct if found, NULL otherwise
	 */
	inline FNamedOnlineSession* GetGameServerSession()
	{
		FScopeLock ScopeLock(&SessionLock);
		for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
		{
			FNamedOnlineSession& Session = Sessions[SearchIndex];
			if (Session.SessionInfo.IsValid())
			{
				FOnlineSessionInfoSteam* SessionInfo = (FOnlineSessionInfoSteam*)Session.SessionInfo.Get();
				if (SessionInfo->SessionType == ESteamSession::AdvertisedSessionHost)
				{
					return &Sessions[SearchIndex];
				}
			}
		}
		return NULL;
	}

	/**
	 * Debug function to make sure that the sessions and lobbies are in sync
	 * Leaves any lobby that doesn't have a session associated with it
	 */
	void SyncLobbies();

	/**
	 * Keep track of lobbies joined
	 *
	 * @param LobbyId lobby being joined
	 */
	void JoinedLobby(const FUniqueNetIdSteam& LobbyId)
	{
		FScopeLock ScopeLock(&JoinedLobbyLock);
		JoinedLobbyList.Add(LobbyId.AsShared());
	}

	/**
	 * Keep track of lobbies left
	 *
	 * @param LobbyId lobby being left
	 */
	void LeftLobby(const FUniqueNetIdSteam& LobbyId)
	{
		FScopeLock ScopeLock(&JoinedLobbyLock);

		for (int i = 0; i < JoinedLobbyList.Num(); i++)
		{
			if (*JoinedLobbyList[i] == LobbyId)
			{
				JoinedLobbyList.RemoveAt(i);
				return;
			}
		}
	}

	/**
	 * Has a particular lobby been joined already
	 *
	 * @param LobbyId lobby to query
	 *
	 * @return true if member of lobby, else false
	 */
	bool IsMemberOfLobby(const FUniqueNetIdSteam& LobbyId)
	{
		FScopeLock ScopeLock(&JoinedLobbyLock);

		for (int i = 0; i < JoinedLobbyList.Num(); i++)
		{
			if (*JoinedLobbyList[i] == LobbyId)
			{
				return true;
			}
		}
		return false;
	}

	/**
	 *	Create the proper connection string so another user can connect to the given session
	 *
	 * @param SessionName the session to create the string for
	 *
	 * @return the proper connection string for this session
	 */
	FString GetSteamConnectionString(FName SessionName);

	/**
	 * Parse the command line for invite/join information at launch
	 */
	void CheckPendingSessionInvite();

	/**
	 * Registers all local players with the current session
	 *
	 * @param Session the session that they are registering in
	 */
	void RegisterLocalPlayers(class FNamedOnlineSession* Session);

	/**
	 * Parses the dedicated server custom name launch argument as specified by the
	 * -SteamServerName= flag.
	 *
	 * @return the custom name if the value of the launch argument is specified and less than 64 characters
	 *			otherwise returns empty string
	 */
	FString GetCustomDedicatedServerName() const;

public:

	virtual ~FOnlineSessionSteam() {}

	virtual FUniqueNetIdPtr CreateSessionIdFromString(const FString& SessionIdStr) override;

	FNamedOnlineSession* GetNamedSession(FName SessionName) override
	{
		FScopeLock ScopeLock(&SessionLock);
		for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
		{
			if (Sessions[SearchIndex].SessionName == SessionName)
			{
				return &Sessions[SearchIndex];
			}
		}
		return NULL;
	}

	virtual void RemoveNamedSession(FName SessionName) override
	{
		FScopeLock ScopeLock(&SessionLock);
		for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
		{
			if (Sessions[SearchIndex].SessionName == SessionName)
			{
				Sessions.RemoveAtSwap(SearchIndex);
				return;
			}
		}
	}

	virtual EOnlineSessionState::Type GetSessionState(FName SessionName) const override
	{
		FScopeLock ScopeLock(&SessionLock);
		for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
		{
			if (Sessions[SearchIndex].SessionName == SessionName)
			{
				return Sessions[SearchIndex].SessionState;
			}
		}

		return EOnlineSessionState::NoSession;
	}

	virtual bool HasPresenceSession() override
	{
		FScopeLock ScopeLock(&SessionLock);
		for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
		{
			if (Sessions[SearchIndex].SessionSettings.bUsesPresence)
			{
				return true;
			}
		}
		
		return false;
	}

	// IOnlineSession
	virtual bool CreateSession(int32 HostingPlayerNum, FName SessionName, const FOnlineSessionSettings& NewSessionSettings) override;
	virtual bool CreateSession(const FUniqueNetId& HostingPlayerId, FName SessionName, const FOnlineSessionSettings& NewSessionSettings) override;
	virtual bool StartSession(FName SessionName) override;
	virtual bool UpdateSession(FName SessionName, FOnlineSessionSettings& UpdatedSessionSettings, bool bShouldRefreshOnlineData = true) override;
	virtual bool EndSession(FName SessionName) override;
	virtual bool DestroySession(FName SessionName, const FOnDestroySessionCompleteDelegate& CompletionDelegate = FOnDestroySessionCompleteDelegate()) override;
	virtual bool IsPlayerInSession(FName SessionName, const FUniqueNetId& UniqueId) override;
	virtual bool StartMatchmaking(const TArray< FUniqueNetIdRef >& LocalPlayers, FName SessionName, const FOnlineSessionSettings& NewSessionSettings, TSharedRef<FOnlineSessionSearch>& SearchSettings) override;
	virtual bool CancelMatchmaking(int32 SearchingPlayerNum, FName SessionName) override;
	virtual bool CancelMatchmaking(const FUniqueNetId& SearchingPlayerId, FName SessionName) override;
	virtual bool FindSessions(int32 SearchingPlayerNum, const TSharedRef<FOnlineSessionSearch>& SearchSettings) override;
	virtual bool FindSessions(const FUniqueNetId& SearchingPlayerId, const TSharedRef<FOnlineSessionSearch>& SearchSettings) override;
	virtual bool FindSessionById(const FUniqueNetId& SearchingUserId, const FUniqueNetId& SessionId, const FUniqueNetId& FriendId, const FOnSingleSessionResultCompleteDelegate& CompletionDelegate) override;
	virtual bool CancelFindSessions() override;
	virtual bool PingSearchResults(const FOnlineSessionSearchResult& SearchResult) override;
	virtual bool JoinSession(int32 PlayerNum, FName SessionName, const FOnlineSessionSearchResult& DesiredSession) override;
	virtual bool JoinSession(const FUniqueNetId& PlayerId, FName SessionName, const FOnlineSessionSearchResult& DesiredSession) override;
	virtual bool FindFriendSession(int32 LocalUserNum, const FUniqueNetId& Friend) override;
	virtual bool FindFriendSession(const FUniqueNetId& LocalUserId, const FUniqueNetId& Friend) override;
	virtual bool FindFriendSession(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& FriendList) override;
	virtual bool SendSessionInviteToFriend(int32 LocalUserNum, FName SessionName, const FUniqueNetId& Friend) override;
	virtual bool SendSessionInviteToFriend(const FUniqueNetId& LocalUserId, FName SessionName, const FUniqueNetId& Friend) override;
	virtual bool SendSessionInviteToFriends(int32 LocalUserNum, FName SessionName, const TArray< FUniqueNetIdRef >& Friends) override;
	virtual bool SendSessionInviteToFriends(const FUniqueNetId& LocalUserId, FName SessionName, const TArray< FUniqueNetIdRef >& Friends) override;
	virtual bool GetResolvedConnectString(FName SessionName, FString& ConnectInfo, FName PortType) override;
	virtual bool GetResolvedConnectString(const FOnlineSessionSearchResult& SearchResult, FName PortType, FString& ConnectInfo) override;
	virtual FOnlineSessionSettings* GetSessionSettings(FName SessionName) override;
	virtual bool RegisterPlayer(FName SessionName, const FUniqueNetId& PlayerId, bool bWasInvited) override;
	virtual bool RegisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players, bool bWasInvited = false) override;
	virtual bool UnregisterPlayer(FName SessionName, const FUniqueNetId& PlayerId) override;
	virtual bool UnregisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players) override;
	virtual void RegisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnRegisterLocalPlayerCompleteDelegate& Delegate) override;
	virtual void UnregisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnUnregisterLocalPlayerCompleteDelegate& Delegate) override;
	virtual int32 GetNumSessions() override;
	virtual void DumpSessionState() override;
};

typedef TSharedPtr<FOnlineSessionSteam, ESPMode::ThreadSafe> FOnlineSessionSteamPtr;
