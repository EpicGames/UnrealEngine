// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "OnlineSharedCloudInterfaceSteam.h"
#include "OnlineSubsystemSteam.h"

FString FOnlineAsyncTaskSteamReadSharedFile::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncTaskSteamReadSharedFile bWasSuccessful: %d Handle: %s"), WasSuccessful(), *SharedHandle.ToDebugString());
}

void FOnlineAsyncTaskSteamReadSharedFile::Tick()
{
	ISteamUtils* SteamUtilsPtr = SteamUtils();
	check(SteamUtilsPtr);

	if (!bInit)
	{
		if (SteamRemoteStorage() && SharedHandle.IsValid())
		{
			if (SteamUser()->BLoggedOn())
			{
				// Actual request to download the file from Steam
				CallbackHandle = SteamRemoteStorage()->UGCDownload(*(UGCHandle_t*)SharedHandle.GetBytes(), 0);

			}
			else
			{
				UE_LOG_ONLINE_CLOUD(Warning, TEXT("Steam user not logged in."));
			}
		}
		else
		{
			UE_LOG_ONLINE_CLOUD(Warning, TEXT("Steam remote storage API disabled."));
		}

		bInit = true;
	}

	if (CallbackHandle != k_uAPICallInvalid)
	{
		bool bFailedCall = false; 

		// Poll for completion status
		bIsComplete = SteamUtilsPtr->IsAPICallCompleted(CallbackHandle, &bFailedCall) ? true : false; 
		if (bIsComplete) 
		{ 
			bool bFailedResult;
			// Retrieve the callback data from the request
			bool bSuccessCallResult = SteamUtilsPtr->GetAPICallResult(CallbackHandle, &CallbackResults, sizeof(CallbackResults), CallbackResults.k_iCallback, &bFailedResult); 
			bWasSuccessful = (bSuccessCallResult ? true : false) &&
				(!bFailedCall ? true : false) &&
				(!bFailedResult ? true : false) &&
				((CallbackResults.m_eResult == k_EResultOK) ? true : false);
		} 
	}
	else
	{
		// Invalid API call
		bIsComplete = true;
		bWasSuccessful = false;
	}
}

void FOnlineAsyncTaskSteamReadSharedFile::Finalize()
{
	FOnlineAsyncTaskSteam::Finalize();

	FOnlineSharedCloudSteamPtr SharedCloud = StaticCastSharedPtr<FOnlineSharedCloudSteam>(Subsystem->GetSharedCloudInterface());
	FCloudFileSteam* SharedFile = SharedCloud->GetSharedCloudFile(SharedHandle);
	if (SharedFile)
	{
		if (bWasSuccessful)
		{
			// Currently don't support greater than 1 chunk (we read everything in at once)
			if (FSharedContentHandleSteam(CallbackResults.m_hFile) == SharedHandle && CallbackResults.m_nSizeInBytes > 0 && CallbackResults.m_nSizeInBytes <= k_unMaxCloudFileChunkSize)
			{
				SharedFile->Data.Empty(CallbackResults.m_nSizeInBytes);
				SharedFile->Data.AddUninitialized(CallbackResults.m_nSizeInBytes);

				uint32 FileOffset = 0;
				// This call only works once per call to UGCDownload()
				if (SteamRemoteStorage()->UGCRead(CallbackResults.m_hFile, SharedFile->Data.GetData(), SharedFile->Data.Num(), FileOffset, k_EUGCRead_ContinueReadingUntilFinished) != CallbackResults.m_nSizeInBytes)
				{
					// Failed to read the data from disk
					SharedFile->Data.Empty();
					bWasSuccessful = false;
				}
			}
			else
			{
				// Bad handle or bad filesize
				bWasSuccessful = false;
			}
		}
	
		SharedFile->AsyncState = bWasSuccessful ? EOnlineAsyncTaskState::Done : EOnlineAsyncTaskState::Failed;
	}
	else
	{
		// No file found
		bWasSuccessful = false;
	}
}

void FOnlineAsyncTaskSteamReadSharedFile::TriggerDelegates()
{
	FOnlineAsyncTaskSteam::TriggerDelegates();

	IOnlineSharedCloudPtr SharedCloudInterface = Subsystem->GetSharedCloudInterface();
	SharedCloudInterface->TriggerOnReadSharedFileCompleteDelegates(bWasSuccessful, SharedHandle);
}

FString FOnlineAsyncTaskSteamWriteSharedFile::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncTaskSteamWriteSharedFile bWasSuccessful:%d UserId:%s FileName:%s Handle:%s"),
								WasSuccessful(), *UserId->ToDebugString(), *FileName, *FSharedContentHandleSteam(CallbackResults.m_hFile).ToDebugString());
}

void FOnlineAsyncTaskSteamWriteSharedFile::Tick()
{
	ISteamUtils* SteamUtilsPtr = SteamUtils();
	check(SteamUtilsPtr);
	
	if (!bInit)
	{
		if (WriteUserFile(*UserId, FileName, Contents))
		{
			// Simply mark the file as shared, will trigger a delegate when upload is complete
			CallbackHandle = SteamRemoteStorage()->FileShare(TCHAR_TO_UTF8(*FileName));
		}

		bInit = true;
	}

	if (CallbackHandle != k_uAPICallInvalid)
	{
		bool bFailedCall = false; 

		// Poll for completion status
		bIsComplete = SteamUtilsPtr->IsAPICallCompleted(CallbackHandle, &bFailedCall) ? true : false; 
		if (bIsComplete) 
		{ 
			bool bFailedResult;
			// Retrieve the callback data from the request
			bool bSuccessCallResult = SteamUtilsPtr->GetAPICallResult(CallbackHandle, &CallbackResults, sizeof(CallbackResults), CallbackResults.k_iCallback, &bFailedResult); 
			bWasSuccessful = (bSuccessCallResult ? true : false) &&
				(!bFailedCall ? true : false) &&
				(!bFailedResult ? true : false) &&
				((CallbackResults.m_eResult == k_EResultOK) ? true : false);
		} 
	}
	else
	{
		// Invalid API call
		bIsComplete = true;
		bWasSuccessful = false;
	}
}
	
void FOnlineAsyncTaskSteamWriteSharedFile::Finalize()
{
	FOnlineAsyncTaskSteam::Finalize();

	if (bWasSuccessful)
	{
		// If the task failed, we'll have no "handle" to associate with the done state
		FOnlineSharedCloudSteamPtr SharedCloud = StaticCastSharedPtr<FOnlineSharedCloudSteam>(Subsystem->GetSharedCloudInterface());
		FSharedContentHandleSteam SharedHandle(CallbackResults.m_hFile);

		// Create the entry to hold the data
		FCloudFileSteam* SharedFile = SharedCloud->GetSharedCloudFile(SharedHandle);
		if (SharedFile)
		{
			SharedFile->Data = Contents;
			SharedFile->AsyncState = EOnlineAsyncTaskState::Done;
		}
	}

	// Done with this copy of the data regardless
	Contents.Empty();
}
	
void FOnlineAsyncTaskSteamWriteSharedFile::TriggerDelegates()
{ 
	FOnlineAsyncTaskSteam::TriggerDelegates();

	IOnlineSharedCloudPtr SharedCloudInterface = Subsystem->GetSharedCloudInterface();

	UGCHandle_t NewHandle = bWasSuccessful ? CallbackResults.m_hFile : k_UGCHandleInvalid;
	TSharedRef<FSharedContentHandle> SharedHandle = MakeShareable(new FSharedContentHandleSteam(NewHandle));

	SharedCloudInterface->TriggerOnWriteSharedFileCompleteDelegates(bWasSuccessful, *UserId, FileName, SharedHandle);
}

FCloudFileSteam* FOnlineSharedCloudSteam::GetSharedCloudFile(const FSharedContentHandle& SharedHandle)
{
	for (int32 FileIdx=0; FileIdx < SharedFileCache.Num(); FileIdx++)
	{
		FCloudFileSteam* SharedFile = SharedFileCache[FileIdx];
		if (SharedFile->SharedHandle == *(FSharedContentHandleSteam*)&SharedHandle)
		{
			return SharedFile;
		}
	}

	// Always create a new one if it doesn't exist
	FCloudFileSteam* NewItem = new FCloudFileSteam(*(FSharedContentHandleSteam*)&SharedHandle);
	int32 FileIdx = SharedFileCache.Add(NewItem);
	return SharedFileCache[FileIdx];
}

bool FOnlineSharedCloudSteam::GetSharedFileContents(const FSharedContentHandle& SharedHandle, TArray<uint8>& FileContents)
{
	FCloudFileSteam* SharedFile = GetSharedCloudFile(SharedHandle);
	if (SharedFile && SharedFile->AsyncState == EOnlineAsyncTaskState::Done && SharedFile->Data.Num() > 0)
	{
		FileContents = SharedFile->Data;
		return true;
	}
	else
	{
		FileContents.Empty();
	}

	return false;
}

bool FOnlineSharedCloudSteam::ClearSharedFiles()
{
	for (int32 FileIdx=0; FileIdx < SharedFileCache.Num(); FileIdx++)
	{
		const FCloudFileSteam* SharedFile = SharedFileCache[FileIdx];
		// If there is an async task outstanding, fail to empty
		if (SharedFile->AsyncState == EOnlineAsyncTaskState::InProgress)
		{
			return false;
		}
	}

	for (int32 FileIdx=0; FileIdx < SharedFileCache.Num(); FileIdx++)
	{
		FCloudFileSteam* SharedFile = SharedFileCache[FileIdx];
		delete SharedFile;
	}

	SharedFileCache.Empty();
	return true;
}

bool FOnlineSharedCloudSteam::ClearSharedFile(const FSharedContentHandle& SharedHandle)
{
	for (int32 FileIdx=0; FileIdx < SharedFileCache.Num(); FileIdx++)
	{
		const FCloudFileSteam* SharedFile = SharedFileCache[FileIdx];
		if (SharedFile->SharedHandle == *(FSharedContentHandleSteam*)&SharedHandle)
		{
			// If there is an async task outstanding, fail to empty
			if (SharedFile->AsyncState != EOnlineAsyncTaskState::InProgress)
			{
				SharedFileCache.RemoveAtSwap(FileIdx);
				delete SharedFile;
				return true;
			}
			break;
		}
	}

	return false;
}

bool FOnlineSharedCloudSteam::ReadSharedFile(const FSharedContentHandle& SharedHandle)
{
	bool bSuccess = false;

	// Create the entry to hold the data
	FCloudFileSteam* SharedFile = GetSharedCloudFile(SharedHandle);
	if (SharedFile)
	{
		SharedFile->AsyncState = EOnlineAsyncTaskState::InProgress;
		SteamSubsystem->QueueAsyncTask(new FOnlineAsyncTaskSteamReadSharedFile(SteamSubsystem, *(FSharedContentHandleSteam*)&SharedHandle));
		bSuccess = true;
	}

	return bSuccess;
}

bool FOnlineSharedCloudSteam::WriteSharedFile(const FUniqueNetId& UserId, const FString& FileName, TArray<uint8>& FileContents)
{
	SteamSubsystem->QueueAsyncTask(new FOnlineAsyncTaskSteamWriteSharedFile(SteamSubsystem, FUniqueNetIdSteam::Cast(UserId), FileName, FileContents));
	return true;
}

void FOnlineSharedCloudSteam::GetDummySharedHandlesForTest(TArray< TSharedRef<FSharedContentHandle> > & OutHandles)
{
	OutHandles.Add(MakeShareable(new FSharedContentHandleSteam(594715184766135714)));
	OutHandles.Add(MakeShareable(new FSharedContentHandleSteam(594715184766136144)));
	OutHandles.Add(MakeShareable(new FSharedContentHandleSteam(594715184766136543)));
	OutHandles.Add(MakeShareable(new FSharedContentHandleSteam(594715184766137039)));
	OutHandles.Add(MakeShareable(new FSharedContentHandleSteam(594715184766137499)));
	OutHandles.Add(MakeShareable(new FSharedContentHandleSteam(594715184766137928)));
	OutHandles.Add(MakeShareable(new FSharedContentHandleSteam(594715184766138377)));
	OutHandles.Add(MakeShareable(new FSharedContentHandleSteam(594715184766138784)));
	OutHandles.Add(MakeShareable(new FSharedContentHandleSteam(594715184766139217)));
	OutHandles.Add(MakeShareable(new FSharedContentHandleSteam(594715184766139630)));
	OutHandles.Add(MakeShareable(new FSharedContentHandleSteam(594715184766140275)));
	OutHandles.Add(MakeShareable(new FSharedContentHandleSteam(594715184766140713)));
	OutHandles.Add(MakeShareable(new FSharedContentHandleSteam(594715184766141131)));
	OutHandles.Add(MakeShareable(new FSharedContentHandleSteam(594715184766141899)));
	OutHandles.Add(MakeShareable(new FSharedContentHandleSteam(594715184766142348)));
}


