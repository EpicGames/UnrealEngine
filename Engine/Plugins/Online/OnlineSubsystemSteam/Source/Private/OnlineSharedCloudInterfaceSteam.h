// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "OnlineUserCloudInterfaceSteam.h"
#include "Interfaces/OnlineSharedCloudInterface.h"

/** 
 *  Async task for reading/downloading a single publicly shared cloud file
 */
class FOnlineAsyncTaskSteamReadSharedFile : public FOnlineAsyncTaskSteam
{
private:

	/** Has this request been started */
	bool bInit;
	/** Steam representation of handle */
	FSharedContentHandleSteam SharedHandle;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamReadSharedFile() : 
		bInit(false),
		SharedHandle(k_UGCHandleInvalid)
	{
	}

PACKAGE_SCOPE:

	/** Remote share request data */
	RemoteStorageDownloadUGCResult_t CallbackResults;

public:

	FOnlineAsyncTaskSteamReadSharedFile(class FOnlineSubsystemSteam* InSubsystem, const FSharedContentHandleSteam& InSharedHandle) :
		FOnlineAsyncTaskSteam(InSubsystem, k_uAPICallInvalid),
		bInit(false),
		SharedHandle(InSharedHandle)
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override;

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override;

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override;
	
	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override;
};

/** 
 *  Async task for writing to disk then sharing a user's cloud file
 */
class FOnlineAsyncTaskSteamWriteSharedFile : public FOnlineAsyncTaskSteamWriteUserFile
{
private:

	/** Has this request been started */
	bool bInit;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamWriteSharedFile() = delete;

PACKAGE_SCOPE:

	/** Remote share request data */
	RemoteStorageFileShareResult_t CallbackResults; 

public:

	FOnlineAsyncTaskSteamWriteSharedFile(class FOnlineSubsystemSteam* InSubsystem, const FUniqueNetIdSteam& InUserId, const FString& InFileName, const TArray<uint8>& InContents) :
		FOnlineAsyncTaskSteamWriteUserFile(InSubsystem, InUserId, InFileName, InContents),
		bInit(false)
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override;

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override;

	/**
	 * Write out the state of the request
	 */
	virtual void Finalize() override;
	
	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override;
};

/**
 * Provides the interface for sharing files already on the cloud with other users
 */
class FOnlineSharedCloudSteam : public IOnlineSharedCloud
{
protected:

	/** Reference to the main Steam subsystem */
	class FOnlineSubsystemSteam* SteamSubsystem;
	/** Array of all files downloaded/cached by the system */
	TArray<struct FCloudFileSteam*> SharedFileCache;

	FOnlineSharedCloudSteam() :
		SteamSubsystem(NULL)
	{
	}

PACKAGE_SCOPE:

	FOnlineSharedCloudSteam(class FOnlineSubsystemSteam* InSubsystem) :
		SteamSubsystem(InSubsystem)
	{
	}

	/** 
	 * **INTERNAL**
	 * Get the file entry related to a shared download
	 *
	 * @param SharedHandle the handle to search for
	 * @return the struct with the metadata about the requested shared data, will always return a valid struct, creating one if necessary
	 *
	 */
	struct FCloudFileSteam* GetSharedCloudFile(const FSharedContentHandle& SharedHandle);

public:

	virtual ~FOnlineSharedCloudSteam() 
	{
		ClearSharedFiles();
	}

	// IOnlineSharedCloud
	virtual bool GetSharedFileContents(const FSharedContentHandle& SharedHandle, TArray<uint8>& FileContents) override;
	virtual bool ClearSharedFiles() override final;
	virtual bool ClearSharedFile(const FSharedContentHandle& SharedHandle) override;
	virtual bool ReadSharedFile(const FSharedContentHandle& SharedHandle) override;
	virtual bool WriteSharedFile(const FUniqueNetId& UserId, const FString& Filename, TArray<uint8>& FileContents) override;
	virtual void GetDummySharedHandlesForTest(TArray< TSharedRef<FSharedContentHandle> > & OutHandles) override;
};

typedef TSharedPtr<FOnlineSharedCloudSteam, ESPMode::ThreadSafe> FOnlineSharedCloudSteamPtr;
