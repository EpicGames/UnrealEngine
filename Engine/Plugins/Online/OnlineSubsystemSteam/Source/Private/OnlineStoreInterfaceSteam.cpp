// Copyright Epic Games, Inc. All Rights Reserved.
#include "OnlineStoreInterfaceSteam.h"
#include "OnlineSubsystemSteam.h"

FOnlineStoreSteam::FOnlineStoreSteam(FOnlineSubsystemSteam* InSteamSubsystem)
	: SteamSubsystem(InSteamSubsystem)
{
	SteamPurchaseInt = StaticCastSharedRef<FOnlinePurchaseSteam>(InSteamSubsystem->GetPurchaseInterface().ToSharedRef());
}

FOnlineStoreSteam::~FOnlineStoreSteam()
{

}

void FOnlineStoreSteam::QueryCategories(const FUniqueNetId& UserId, const FOnQueryOnlineStoreCategoriesComplete& Delegate)
{
	SteamSubsystem->ExecuteNextTick([Delegate]() {
		Delegate.ExecuteIfBound(true, TEXT(""));
	});
}

void FOnlineStoreSteam::GetCategories(TArray<FOnlineStoreCategory>& OutCategories) const
{
	return SteamPurchaseInt->GetStoreCategories(OutCategories);
}

void FOnlineStoreSteam::QueryOffersByFilter(const FUniqueNetId& UserId, const FOnlineStoreFilter& Filter, const FOnQueryOnlineStoreOffersComplete& Delegate)
{
	SteamSubsystem->ExecuteNextTick([Delegate]() {
		TArray<FUniqueOfferId> OfferIds;
		Delegate.ExecuteIfBound(true, OfferIds, TEXT(""));
	});
}

void FOnlineStoreSteam::QueryOffersById(const FUniqueNetId& UserId, const TArray<FUniqueOfferId>& OfferIds, const FOnQueryOnlineStoreOffersComplete& Delegate)
{
	SteamSubsystem->ExecuteNextTick([Delegate]() {
		TArray<FUniqueOfferId> OfferIds;
		Delegate.ExecuteIfBound(true, OfferIds, TEXT(""));
	});
}

void FOnlineStoreSteam::GetOffers(TArray<FOnlineStoreOfferRef>& OutOffers) const
{
	return SteamPurchaseInt->GetStoreOffers(OutOffers);
}

TSharedPtr<FOnlineStoreOffer> FOnlineStoreSteam::GetOffer(const FUniqueOfferId& OfferId) const
{
	return SteamPurchaseInt->GetStoreOffer(OfferId);
}

