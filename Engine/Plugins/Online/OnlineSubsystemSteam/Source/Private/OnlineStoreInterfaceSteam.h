// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemSteamPrivate.h"
#include "Interfaces/OnlineStoreInterfaceV2.h"
#include "OnlinePurchaseInterfaceSteam.h"
#include "OnlineSubsystemSteamPackage.h"

class FOnlineSubsystemSteam;
class FStoreTaskBase;

/**
 * Implementation for online store via Steam
 */
class FOnlineStoreSteam :
	public IOnlineStoreV2,
	public TSharedFromThis<FOnlineStoreSteam, ESPMode::ThreadSafe>
{
public:
	FOnlineStoreSteam(FOnlineSubsystemSteam* InSteamSubsystem);
	virtual ~FOnlineStoreSteam();

public:// IOnlineStoreV2
	virtual void QueryCategories(const FUniqueNetId& UserId, const FOnQueryOnlineStoreCategoriesComplete& Delegate) override;
	virtual void GetCategories(TArray<FOnlineStoreCategory>& OutCategories) const override;
	virtual void QueryOffersByFilter(const FUniqueNetId& UserId, const FOnlineStoreFilter& Filter, const FOnQueryOnlineStoreOffersComplete& Delegate) override;
	virtual void QueryOffersById(const FUniqueNetId& UserId, const TArray<FUniqueOfferId>& OfferIds, const FOnQueryOnlineStoreOffersComplete& Delegate) override;
	virtual void GetOffers(TArray<FOnlineStoreOfferRef>& OutOffers) const override;
	virtual TSharedPtr<FOnlineStoreOffer> GetOffer(const FUniqueOfferId& OfferId) const override;


PACKAGE_SCOPE:
	FOnlineSubsystemSteam* SteamSubsystem;
	TSharedPtr<FOnlinePurchaseSteam> SteamPurchaseInt;
	TArray<FOnlineStoreCategory> CachedCategories;
	TMap<FUniqueOfferId, FOnlineStoreOfferRef> CachedOffers;
};
