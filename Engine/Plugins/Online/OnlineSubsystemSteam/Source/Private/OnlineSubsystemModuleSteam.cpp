// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemSteamModule.h"
#include "OnlineSubsystemSteam.h"
#include "SteamSharedModule.h"
#include "Misc/CommandLine.h"

IMPLEMENT_MODULE(FOnlineSubsystemSteamModule, OnlineSubsystemSteam);

//HACKTASTIC (Needed to keep delete function from being stripped out and crashing when protobuffers deallocate memory)
void* HackDeleteFunctionPointer = (void*)(void(*)(void*))(::operator delete[]);

/**
 * Class responsible for creating instance(s) of the subsystem
 */
class FOnlineFactorySteam final : public IOnlineFactory
{

private:

	/** Single instantiation of the STEAM interface */
	static FOnlineSubsystemSteamPtr SteamSingleton;

	void DestroySubsystem()
	{
		if (SteamSingleton.IsValid())
		{
			SteamSingleton->Shutdown();
			SteamSingleton = nullptr;
		}
	}

public:

	FOnlineFactorySteam() {}
	virtual ~FOnlineFactorySteam() 
	{
		DestroySubsystem();
	}

	virtual IOnlineSubsystemPtr CreateSubsystem(FName InstanceName)
	{
		if (!SteamSingleton.IsValid())
		{
			SteamSingleton = MakeShared<FOnlineSubsystemSteam, ESPMode::ThreadSafe>(InstanceName);
			if (SteamSingleton->IsEnabled())
			{
				if(!SteamSingleton->Init())
				{
					UE_LOG_ONLINE(Warning, TEXT("Steam API failed to initialize!"));
					DestroySubsystem();
				}
			}
			else
			{
				UE_CLOG_ONLINE(IsRunningDedicatedServer() || IsRunningGame(), Warning, TEXT("Steam API disabled!"));
				DestroySubsystem();
			}

			return SteamSingleton;
		}

		UE_LOG_ONLINE(Warning, TEXT("Can't create more than one instance of Steam online subsystem!"));
		return nullptr;
	}
};

FOnlineSubsystemSteamPtr FOnlineFactorySteam::SteamSingleton = nullptr;


void FOnlineSubsystemSteamModule::StartupModule()
{
	if (FParse::Param(FCommandLine::Get(), TEXT("nosteam")))
	{
		UE_LOG_ONLINE(Warning, TEXT("Steam subsystem has been disabled by command line (-nosteam)"));
		return;
	}
	
	FSteamSharedModule& SharedModule = FSteamSharedModule::Get();

	// Load the Steam modules before first call to API
	if (SharedModule.AreSteamDllsLoaded())
	{
		// Create and register our singleton factory with the main online subsystem for easy access
		SteamFactory = new FOnlineFactorySteam();

		FOnlineSubsystemModule& OSS = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
		OSS.RegisterPlatformService(STEAM_SUBSYSTEM, SteamFactory);
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("Steam SDK %s libraries not present at %s or failed to load!"), STEAM_SDK_VER, *SharedModule.GetSteamModulePath());
	}
}

void FOnlineSubsystemSteamModule::ShutdownModule()
{
	FOnlineSubsystemModule& OSS = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
	OSS.UnregisterPlatformService(STEAM_SUBSYSTEM);

	delete SteamFactory;
	SteamFactory = nullptr;
}
