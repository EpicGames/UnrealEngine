// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemSteam.h"
#include "HAL/FileManager.h"
#include "HAL/RunnableThread.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/StringBuilder.h"

#include "OnlineSessionInterfaceSteam.h"
#include "OnlineIdentityInterfaceSteam.h"
#include "OnlinePresenceInterfaceSteam.h"
#include "OnlineFriendsInterfaceSteam.h"
#include "OnlineSharedCloudInterfaceSteam.h"
#include "OnlineLeaderboardInterfaceSteam.h"
#include "VoiceInterfaceSteam.h"
#include "OnlineExternalUIInterfaceSteam.h"
#include "OnlineAchievementsInterfaceSteam.h"
#include "OnlineAuthInterfaceSteam.h"
#include "OnlineAuthInterfaceUtilsSteam.h"
#include "OnlineEncryptedAppTicketInterfaceSteam.h"
#include "OnlinePurchaseInterfaceSteam.h"
#include "OnlineStoreInterfaceSteam.h"
#include "VoiceInterfaceSteam.h"

#include "SteamSharedModule.h"
#include <steam/isteamapps.h>
#include <steam/isteamgameserverstats.h>

/* Specify this define in your Target.cs for your project
 *
 * This helps the SteamAPI find your project on shipping builds
 * if your game is launched outside of Steam.
 */
#ifndef UE_PROJECT_STEAMSHIPPINGID
#define UE_PROJECT_STEAMSHIPPINGID 0
#endif

namespace FNetworkProtocolTypes
{
	const FLazyName Steam(TEXT("Steam"));
}

namespace OSSConsoleVariables
{
	TAutoConsoleVariable<int32> CVarSteamInitServerOnClient(
			TEXT("OSS.SteamInitServerOnClient"),
			0,
			TEXT("Whether or not to initialize the Steam server interface on clients (default false)"),
			ECVF_Default | ECVF_Cheat);

#if !UE_BUILD_SHIPPING
	/** CVar used by NetcodeUnitTest, to force-enable Steam within the unit test commandlet */
	TAutoConsoleVariable<int32> CVarSteamUnitTest(
			TEXT("OSS.SteamUnitTest"),
			0,
			TEXT("Whether or not Steam is being force-enabled by NetcodeUnitTest"),
			ECVF_Default);
#endif
}


extern "C" 
{ 
	static void __cdecl SteamworksWarningMessageHook(int Severity, const char *Message); 
	static void __cdecl SteamworksWarningMessageHookNoOp(int Severity, const char *Message);
}

/** 
 * Callback function into Steam error messaging system
 * @param Severity - error level
 * @param Message - message from Steam
 */
static void __cdecl SteamworksWarningMessageHook(int Severity, const char *Message)
{
	const TCHAR *MessageType;
	switch (Severity)
	{
		case 0: MessageType = TEXT("message"); break;
		case 1: MessageType = TEXT("warning"); break;
		default: MessageType = TEXT("notification"); break;  // Unknown severity; new SDK?
	}
	UE_LOG_ONLINE(Warning, TEXT("Steamworks SDK %s: %s"), MessageType, UTF8_TO_TCHAR(Message));
}

/** 
 * Callback function into Steam error messaging system that outputs nothing
 * @param Severity - error level
 * @param Message - message from Steam
 */
static void __cdecl SteamworksWarningMessageHookNoOp(int Severity, const char *Message)
{
	// no-op.
}

/**
 * Get relaunch settings from OnlineSubsystemSteam configuration
 *
 * @param RequireRelaunch enforce the Steam client running precondition
 * @param RelaunchAppId appid to launch when the Steam client is loaded
 *
 * @return if this sequence completed without any serious errors
 */
bool GetRelaunchSettings(bool& RequireRelaunch, int32& RelaunchAppId)
{
#if !UE_BUILD_SHIPPING && !UE_BUILD_SHIPPING_WITH_EDITOR
	// Get SteamDevAppId and set it as RelaunchAppId
	if (!GConfig->GetInt(TEXT("OnlineSubsystemSteam"), TEXT("SteamDevAppId"), RelaunchAppId, GEngineIni))
	{
		UE_LOG_ONLINE(Warning, TEXT("Missing SteamDevAppId key in OnlineSubsystemSteam of DefaultEngine.ini"));
		return false;
	}

	// Should the game force a relaunch in Steam if the client isn't already loaded
	if (!GConfig->GetBool(TEXT("OnlineSubsystemSteam"), TEXT("bRelaunchInSteam"), RequireRelaunch, GEngineIni))
	{
		UE_LOG_ONLINE(Warning, TEXT("Missing bRelaunchInSteam key in OnlineSubsystemSteam of DefaultEngine.ini"));
	}

#else
	// Always check against the Steam client when shipping
	RequireRelaunch = true;
	RelaunchAppId = UE_PROJECT_STEAMSHIPPINGID;
#endif

	return true;
}

FOnlineAuthSteamPtr FOnlineSubsystemSteam::GetAuthInterface() const
{
	return AuthInterface;
}

FOnlineAuthSteamUtilsPtr FOnlineSubsystemSteam::GetAuthInterfaceUtils() const
{
	return AuthInterfaceUtils;
}

FOnlinePingSteamPtr FOnlineSubsystemSteam::GetPingInterface() const
{
	return PingInterface;
}
void FOnlineSubsystemSteam::SetPingInterface(FOnlinePingSteamPtr InPingInterface)
{
	PingInterface = InPingInterface;
}

FOnlineEncryptedAppTicketSteamPtr FOnlineSubsystemSteam::GetEncryptedAppTicketInterface() const
{
	return EncryptedAppTicketInterface;
}

IOnlineSessionPtr FOnlineSubsystemSteam::GetSessionInterface() const
{
	return SessionInterface;
}

IOnlineFriendsPtr FOnlineSubsystemSteam::GetFriendsInterface() const
{
	return FriendInterface;
}

IOnlineGroupsPtr FOnlineSubsystemSteam::GetGroupsInterface() const
{
	return nullptr;
}

IOnlinePartyPtr FOnlineSubsystemSteam::GetPartyInterface() const
{
	return nullptr;
}

IOnlineSharedCloudPtr FOnlineSubsystemSteam::GetSharedCloudInterface() const
{
	return SharedCloudInterface;
}

IOnlineUserCloudPtr FOnlineSubsystemSteam::GetUserCloudInterface() const
{
	return UserCloudInterface;
}

IOnlineLeaderboardsPtr FOnlineSubsystemSteam::GetLeaderboardsInterface() const
{
	return LeaderboardsInterface;
}

IOnlineVoicePtr FOnlineSubsystemSteam::GetVoiceInterface() const
{
#if WITH_ENGINE
	if (VoiceInterface.IsValid() && !bVoiceInterfaceInitialized)
	{
		if (!VoiceInterface->Init())
		{
			VoiceInterface = nullptr;
		}

		bVoiceInterfaceInitialized = true;
	}
#endif //WITH_ENGINE

	return VoiceInterface;
}

IOnlineExternalUIPtr FOnlineSubsystemSteam::GetExternalUIInterface() const
{
	return ExternalUIInterface;
}

IOnlineTimePtr FOnlineSubsystemSteam::GetTimeInterface() const
{
	return nullptr;
}

IOnlineIdentityPtr FOnlineSubsystemSteam::GetIdentityInterface() const
{
	return IdentityInterface;
}

IOnlineTitleFilePtr FOnlineSubsystemSteam::GetTitleFileInterface() const
{
	return nullptr;
}

IOnlineEntitlementsPtr FOnlineSubsystemSteam::GetEntitlementsInterface() const
{
	return nullptr;
}

IOnlineEventsPtr FOnlineSubsystemSteam::GetEventsInterface() const
{
	return nullptr;
}

IOnlineAchievementsPtr FOnlineSubsystemSteam::GetAchievementsInterface() const
{
	return AchievementsInterface;
}

IOnlineSharingPtr FOnlineSubsystemSteam::GetSharingInterface() const
{
	return nullptr;
}

IOnlineUserPtr FOnlineSubsystemSteam::GetUserInterface() const
{
	return nullptr;
}

IOnlineMessagePtr FOnlineSubsystemSteam::GetMessageInterface() const
{
	return nullptr;
}

IOnlinePresencePtr FOnlineSubsystemSteam::GetPresenceInterface() const
{
	return PresenceInterface;
}

IOnlinePurchasePtr FOnlineSubsystemSteam::GetPurchaseInterface() const
{
	return PurchaseInterface;
}

IOnlineStoreV2Ptr FOnlineSubsystemSteam::GetStoreV2Interface() const
{
	return StoreInterface;
}

IOnlineChatPtr FOnlineSubsystemSteam::GetChatInterface() const
{
	return nullptr;
}

IOnlineStatsPtr FOnlineSubsystemSteam::GetStatsInterface() const
{
	return nullptr;
}

IOnlineTurnBasedPtr FOnlineSubsystemSteam::GetTurnBasedInterface() const
{
	return nullptr;
}

IOnlineTournamentPtr FOnlineSubsystemSteam::GetTournamentInterface() const
{
	return nullptr;
}

void FOnlineSubsystemSteam::QueueAsyncTask(FOnlineAsyncTask* AsyncTask)
{
	check(OnlineAsyncTaskThreadRunnable);
	OnlineAsyncTaskThreadRunnable->AddToInQueue(AsyncTask);
}

void FOnlineSubsystemSteam::QueueAsyncOutgoingItem(FOnlineAsyncItem* AsyncItem)
{
	check(OnlineAsyncTaskThreadRunnable);
	OnlineAsyncTaskThreadRunnable->AddToOutQueue(AsyncItem);
}

bool FOnlineSubsystemSteam::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSubsystemSteam_Tick);

	if (!FOnlineSubsystemImpl::Tick(DeltaTime))
	{
		return false;
	}

	if (OnlineAsyncTaskThreadRunnable)
	{
		OnlineAsyncTaskThreadRunnable->GameTick();
	}

	if (SessionInterface.IsValid())
	{
		SessionInterface->Tick(DeltaTime);
	}

#if WITH_ENGINE
	if (VoiceInterface.IsValid() && bVoiceInterfaceInitialized)
	{
		VoiceInterface->Tick(DeltaTime);
	}
#endif //WITH_ENGINE

	if (AuthInterface.IsValid())
	{
		AuthInterface->Tick(DeltaTime);
	}

	return true;
}

bool FOnlineSubsystemSteam::Init()
{
	bool bRelaunchInSteam = false;
	int RelaunchAppId = 0;

	if (!GetRelaunchSettings(bRelaunchInSteam, RelaunchAppId))
	{
		UE_LOG_ONLINE(Warning, TEXT("Could not set up the steam environment! Falling back to another OSS."));
		return false;
	}

	const bool bIsServer = IsRunningDedicatedServer();
	bool bInitServerOnClient = false;
	GConfig->GetBool(TEXT("OnlineSubsystemSteam"), TEXT("bInitServerOnClient"), bInitServerOnClient, GEngineIni);
	bool bAttemptServerInit = bIsServer || !!OSSConsoleVariables::CVarSteamInitServerOnClient.GetValueOnGameThread() || bInitServerOnClient;

	UE_LOG_ONLINE(Verbose, TEXT("Steam: Starting SteamWorks. Client [%d] Server [%d]"), !bIsServer, bAttemptServerInit);
	
	// Don't initialize the Steam Client API if we are launching as a server
	bool bClientInitSuccess = !bIsServer ? InitSteamworksClient(bRelaunchInSteam, RelaunchAppId) : true;

	// Initialize the Steam Server API if this is a dedicated server or servers should initialize on clients
	bool bServerInitSuccess = bAttemptServerInit ? (InitSteamworksServer()) : true;

	if (bClientInitSuccess && bServerInitSuccess)
	{
		// Create the online async task thread
		OnlineAsyncTaskThreadRunnable = new FOnlineAsyncTaskManagerSteam(this);
		check(OnlineAsyncTaskThreadRunnable);		
		OnlineAsyncTaskThread = FRunnableThread::Create(OnlineAsyncTaskThreadRunnable, *FString::Printf(TEXT("OnlineAsyncTaskThreadSteam %s"), *InstanceName.ToString()), 128 * 1024, TPri_Normal);
		check(OnlineAsyncTaskThread);
		UE_LOG_ONLINE(Verbose, TEXT("Created thread (ID:%d)."), OnlineAsyncTaskThread->GetThreadID() );

		SessionInterface = MakeShareable(new FOnlineSessionSteam(this));
		SessionInterface->CheckPendingSessionInvite();

		IdentityInterface = MakeShareable(new FOnlineIdentitySteam(this));

		PresenceInterface = MakeShareable(new FOnlinePresenceSteam(this));
		
		AuthInterfaceUtils = MakeShareable(new FOnlineAuthUtilsSteam());
		AuthInterface = MakeShareable(new FOnlineAuthSteam(this, AuthInterfaceUtils));

		if (!bIsServer)
		{
			FriendInterface = MakeShareable(new FOnlineFriendsSteam(this));
			UserCloudInterface = MakeShareable(new FOnlineUserCloudSteam(this));
			SharedCloudInterface = MakeShareable(new FOnlineSharedCloudSteam(this));
			LeaderboardsInterface = MakeShareable(new FOnlineLeaderboardsSteam(this));
#if WITH_ENGINE
			VoiceInterface = MakeShareable(new FOnlineVoiceSteam(this));
#endif //WITH_ENGINE
			ExternalUIInterface = MakeShareable(new FOnlineExternalUISteam(this));
			AchievementsInterface = MakeShareable(new FOnlineAchievementsSteam(this));
			EncryptedAppTicketInterface = MakeShareable(new FOnlineEncryptedAppTicketSteam(this));
			PurchaseInterface = MakeShareable(new FOnlinePurchaseSteam(this));
			StoreInterface = MakeShareable(new FOnlineStoreSteam(this));
		}
		else
		{
#if WITH_ENGINE
			// Need a voice interface here to serialize packets but NOT add to VoiceData.RemotePackets
			VoiceInterface = MakeShareable(new FOnlineVoiceSteam(this));
#endif //WITH_ENGINE
		}
	}
	else
	{
		// If the client succeeded, but the server didn't, this could be because there's a server and client running on the same machine - inform the user
		if (bClientInitSuccess)
		{
			UE_LOG_ONLINE(Warning, TEXT("Failed to initialize Steam, this could be due to a Steam server and client running on the same machine. Try running with -NOSTEAM on the cmdline to disable."));
		}
		Shutdown();
	}

	return bClientInitSuccess && bServerInitSuccess;
}

bool FOnlineSubsystemSteam::Shutdown()
{
	UE_LOG_ONLINE(Display, TEXT("OnlineSubsystemSteam::Shutdown()"));

	FOnlineSubsystemImpl::Shutdown();

	if (OnlineAsyncTaskThread)
	{
		// Destroy the online async task thread
		delete OnlineAsyncTaskThread;
		OnlineAsyncTaskThread = nullptr;
	}

	if (OnlineAsyncTaskThreadRunnable)
	{
		delete OnlineAsyncTaskThreadRunnable;
		OnlineAsyncTaskThreadRunnable = nullptr;
	}

#if WITH_ENGINE
	if (VoiceInterface.IsValid() && bVoiceInterfaceInitialized) {
		VoiceInterface->Shutdown();
	}
#endif //WITH_ENGINE

#define DESTRUCT_INTERFACE(Interface) \
	if (Interface.IsValid()) \
	{ \
		ensure(Interface.IsUnique()); \
		Interface = nullptr; \
	}

	// Destruct the interfaces
	DESTRUCT_INTERFACE(EncryptedAppTicketInterface);
	DESTRUCT_INTERFACE(AchievementsInterface);
	DESTRUCT_INTERFACE(ExternalUIInterface);
	DESTRUCT_INTERFACE(VoiceInterface);
	DESTRUCT_INTERFACE(LeaderboardsInterface);
	DESTRUCT_INTERFACE(SharedCloudInterface);
	DESTRUCT_INTERFACE(UserCloudInterface);
	DESTRUCT_INTERFACE(FriendInterface);
	DESTRUCT_INTERFACE(IdentityInterface);
	DESTRUCT_INTERFACE(AuthInterface);
	DESTRUCT_INTERFACE(AuthInterfaceUtils);
	DESTRUCT_INTERFACE(PingInterface);
	DESTRUCT_INTERFACE(SessionInterface);
	DESTRUCT_INTERFACE(PresenceInterface);
	DESTRUCT_INTERFACE(StoreInterface);
	DESTRUCT_INTERFACE(PurchaseInterface);

#undef DESTRUCT_INTERFACE

	ClearUserCloudFiles();

	ShutdownSteamworks();

	return true;
}

bool FOnlineSubsystemSteam::IsEnabled() const
{
	if (bSteamworksClientInitialized || bSteamworksGameServerInitialized)
	{
		return true;
	}

	// Check the ini for disabling Steam
	bool bEnableSteam = FOnlineSubsystemImpl::IsEnabled();
	if (bEnableSteam)
	{
#if UE_EDITOR
		if (bEnableSteam)
		{
			bEnableSteam = IsRunningDedicatedServer() || IsRunningGame();
		}
#endif

#if !UE_BUILD_SHIPPING
		// Force-enable Steam for NetcodeUnitTest
		if (!!OSSConsoleVariables::CVarSteamUnitTest.GetValueOnGameThread())
		{
			bEnableSteam = true;
		}
#endif
	}

	return bEnableSteam;
}

bool FOnlineSubsystemSteam::InitSteamworksClient(bool bRelaunchInSteam, int32 SteamAppId)
{
	bSteamworksClientInitialized = false;

	// If the game was not launched from within Steam, but is supposed to, trigger a Steam launch and exit
	if (bRelaunchInSteam && SteamAppId != 0 && SteamAPI_RestartAppIfNecessary(SteamAppId))
	{
		if (FPlatformProperties::IsGameOnly() || FPlatformProperties::IsServerOnly())
		{
			UE_LOG_ONLINE(Log, TEXT("Game restarting within Steam client, exiting"));
			FPlatformMisc::RequestExit(false);
		}

		return bSteamworksClientInitialized;
	}
	// Otherwise initialize as normal
	else
	{
		SteamAPIClientHandle = FSteamSharedModule::Get().ObtainSteamClientInstanceHandle();
		// Steamworks needs to initialize as close to start as possible, so it can hook its overlay into Direct3D, etc.
		bSteamworksClientInitialized = (SteamAPIClientHandle.IsValid() ? true : false);

		// Test all the Steam interfaces
#define GET_STEAMWORKS_INTERFACE(Interface) \
		if (Interface() == nullptr) \
		{ \
			UE_LOG_ONLINE(Warning, TEXT("Steamworks: %s() failed!"), TEXT(#Interface)); \
			bSteamworksClientInitialized = false; \
		} \

		// GSteamUtils
		GET_STEAMWORKS_INTERFACE(SteamUtils);
		// GSteamUser
		GET_STEAMWORKS_INTERFACE(SteamUser);
		// GSteamFriends
		GET_STEAMWORKS_INTERFACE(SteamFriends);
		// GSteamRemoteStorage
		GET_STEAMWORKS_INTERFACE(SteamRemoteStorage);
		// GSteamUserStats
		GET_STEAMWORKS_INTERFACE(SteamUserStats);
		// GSteamMatchmakingServers
		GET_STEAMWORKS_INTERFACE(SteamMatchmakingServers);
		// GSteamApps
		GET_STEAMWORKS_INTERFACE(SteamApps);
		// GSteamNetworking
		GET_STEAMWORKS_INTERFACE(SteamNetworking);
		// GSteamMatchmaking
		GET_STEAMWORKS_INTERFACE(SteamMatchmaking);

#undef GET_STEAMWORKS_INTERFACE
	}

	if (bSteamworksClientInitialized)
	{
		GameServerGamePort = SteamAPIClientHandle->GetGamePort();

		bool bIsSubscribed = true;
		if (FPlatformProperties::IsGameOnly() || FPlatformProperties::IsServerOnly())
		{
			bIsSubscribed = SteamApps()->BIsSubscribed();
		}

		// Make sure the Steam user has valid access to the game
		if (bIsSubscribed)
		{
			UE_LOG_ONLINE(Log, TEXT("Steam User is subscribed %i"), bSteamworksClientInitialized);
			if (SteamUtils())
			{
				SteamAppID = SteamUtils()->GetAppID();
				SteamUtils()->SetWarningMessageHook(SteamworksWarningMessageHook);
			}
		}
		else
		{
			UE_LOG_ONLINE(Error, TEXT("Steam User is NOT subscribed, exiting."));
			bSteamworksClientInitialized = false;
			FPlatformMisc::RequestExit(false);
		}
	}

	UE_LOG_ONLINE(Log, TEXT("[AppId: %d] Client API initialized %i"), GetSteamAppId(), bSteamworksClientInitialized);
	return bSteamworksClientInitialized;
}

bool FOnlineSubsystemSteam::InitSteamworksServer()
{
	SteamAPIServerHandle = FSteamSharedModule::Get().ObtainSteamServerInstanceHandle();
	bSteamworksGameServerInitialized = (SteamAPIServerHandle.IsValid());

	if (bSteamworksGameServerInitialized)
	{
		// Grab the port values so that we save them.
		GameServerGamePort = SteamAPIServerHandle->GetGamePort();
		GameServerQueryPort = SteamAPIServerHandle->GetQueryPort();

		// Test all the Steam interfaces
		#define GET_STEAMWORKS_INTERFACE(Interface) \
		if (Interface() == nullptr) \
		{ \
			UE_LOG_ONLINE(Warning, TEXT("Steamworks: %s() failed!"), TEXT(#Interface)); \
			bSteamworksGameServerInitialized = false; \
		} \

		// NOTE: It's not possible for >some< of the interfaces to initialize, and others fail; it's all or none
		GET_STEAMWORKS_INTERFACE(SteamGameServer);
		GET_STEAMWORKS_INTERFACE(SteamGameServerStats);
		GET_STEAMWORKS_INTERFACE(SteamGameServerNetworking);
		GET_STEAMWORKS_INTERFACE(SteamGameServerUtils);

		#undef GET_STEAMWORKS_INTERFACE
	}

	if (SteamGameServerUtils() != nullptr)
	{
		SteamAppID = SteamGameServerUtils()->GetAppID();
		SteamGameServerUtils()->SetWarningMessageHook(SteamworksWarningMessageHook);
	}

	UE_LOG_ONLINE(Log, TEXT("[AppId: %d] Game Server API initialized %i"), GetSteamAppId(), bSteamworksGameServerInitialized);
	return bSteamworksGameServerInitialized;
}

void FOnlineSubsystemSteam::ShutdownSteamworks()
{
	if (bSteamworksGameServerInitialized)
	{
		SteamAPIServerHandle.Reset();
		if (SessionInterface.IsValid())
		{
			SessionInterface->GameServerSteamId = nullptr;
			SessionInterface->bSteamworksGameServerConnected = false;
		}
		bSteamworksGameServerInitialized = false;
	}

	if (bSteamworksClientInitialized)
	{
		SteamAPIClientHandle.Reset();
		bSteamworksClientInitialized = false;
	}
}

bool FOnlineSubsystemSteam::IsLocalPlayer(const FUniqueNetId& UniqueId) const
{
	ISteamUser* SteamUserPtr = SteamUser();
	return SteamUserPtr && SteamUserPtr->GetSteamID() == (const FUniqueNetIdSteam&)UniqueId;
}

FOnlineLeaderboardsSteam * FOnlineSubsystemSteam::GetInternalLeaderboardsInterface()
{
	return LeaderboardsInterface.Get();
}

FSteamUserCloudData* FOnlineSubsystemSteam::GetUserCloudEntry(const FUniqueNetId& UserId)
{
	FScopeLock ScopeLock(&UserCloudDataLock);
	for (int32 UserIdx = 0; UserIdx < UserCloudData.Num(); UserIdx++)
	{
		FSteamUserCloudData* UserMetadata = UserCloudData[UserIdx];
		if (*UserMetadata->UserId == UserId)
		{
			return UserMetadata;
		}
	}

	// Always create a new one if it doesn't exist
	const FUniqueNetIdSteam& SteamUserId = FUniqueNetIdSteam::Cast(UserId);
	FSteamUserCloudData* NewItem = new FSteamUserCloudData(SteamUserId);
	int32 UserIdx = UserCloudData.Add(NewItem);
	return UserCloudData[UserIdx];
}

bool FOnlineSubsystemSteam::ClearUserCloudMetadata(const FUniqueNetId& UserId, const FString& FileName)
{
	if (FileName.Len() > 0)
	{
		FScopeLock ScopeLock(&UserCloudDataLock);
		// Search for the specified file
		FSteamUserCloudData* UserCloud = GetUserCloudEntry(UserId);
		if (UserCloud)
		{
			UserCloud->ClearMetadata(FileName);
		}
	}

	return true;
}

void FOnlineSubsystemSteam::ClearUserCloudFiles()
{
	FScopeLock ScopeLock(&UserCloudDataLock);
	for (int32 UserIdx = 0; UserIdx < UserCloudData.Num(); UserIdx++)
	{
		FSteamUserCloudData* UserCloud = UserCloudData[UserIdx];
		delete UserCloud;
	}

	UserCloudData.Empty();
}

static FDelegateHandle GOnEnumerateUserFilesCompleteDelegateHandle;

TMap<IOnlineUserCloud*, FDelegateHandle> GPerCloudDeleteFromEnumerateUserFilesCompleteDelegateHandles;

static void DeleteFromEnumerateUserFilesComplete(bool bWasSuccessful, const FUniqueNetId& UserId)
{
	IOnlineSubsystem* OnlineSub = IOnlineSubsystem::Get();
	check(OnlineSub);

	IOnlineUserCloudPtr UserCloud = OnlineSub->GetUserCloudInterface();

	UserCloud->ClearOnEnumerateUserFilesCompleteDelegate_Handle(GOnEnumerateUserFilesCompleteDelegateHandle);
	GPerCloudDeleteFromEnumerateUserFilesCompleteDelegateHandles.Remove(UserCloud.Get());
	if (bWasSuccessful)
	{
		TArray<FCloudFileHeader> UserFiles;
		UserCloud->GetUserFileList(UserId, UserFiles);

		for (int32 Idx = 0; Idx < UserFiles.Num(); Idx++)
		{
			UserCloud->DeleteUserFile(UserId, UserFiles[Idx].FileName, true, true);
		}
	}
}

bool FOnlineSubsystemSteam::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FOnlineSubsystemImpl::Exec(InWorld, Cmd, Ar))
	{
		return true;
	}

	bool bWasHandled = false;
	if (FParse::Command(&Cmd, TEXT("DELETECLOUDFILES")))
	{
		IOnlineUserCloudPtr UserCloud = GetUserCloudInterface();

		FUniqueNetIdSteamRef SteamId = FUniqueNetIdSteam::Create(SteamUser()->GetSteamID());

		FOnEnumerateUserFilesCompleteDelegate Delegate = FOnEnumerateUserFilesCompleteDelegate::CreateStatic(&DeleteFromEnumerateUserFilesComplete);
		GPerCloudDeleteFromEnumerateUserFilesCompleteDelegateHandles.Add(UserCloud.Get(), UserCloud->AddOnEnumerateUserFilesCompleteDelegate_Handle(Delegate));
		UserCloud->EnumerateUserFiles(*SteamId);
		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("SYNCLOBBIES")))
	{
		if (SessionInterface.IsValid())
		{
			SessionInterface->SyncLobbies();
			bWasHandled = true;
		}
	}
	else if (FParse::Command(&Cmd, TEXT("AUTH")))
	{
		if (AuthInterface.IsValid())
		{
			bWasHandled = AuthInterface->Exec(Cmd);
		}
	}

	return bWasHandled;
}

FString FOnlineSubsystemSteam::GetAppId() const
{
	return FString::Printf(TEXT("%d"),GetSteamAppId());
}

FText FOnlineSubsystemSteam::GetOnlineServiceName() const
{
	return NSLOCTEXT("OnlineSubsystemSteam", "OnlineServiceName", "Steam");
}