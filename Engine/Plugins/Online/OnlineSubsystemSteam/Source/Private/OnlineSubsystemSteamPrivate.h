// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystem.h"


#define INVALID_INDEX -1

/** URL Prefix when using Steam socket connection */
#define STEAM_URL_PREFIX TEXT("steam.")

/** pre-pended to all steam logging */
#undef ONLINE_LOG_PREFIX
#define ONLINE_LOG_PREFIX TEXT("STEAM: ")

THIRD_PARTY_INCLUDES_START
// IWYU pragma: begin_exports
#include "steam/steam_api.h"
#include "steam/steam_gameserver.h"
#include "steam/isteamnetworkingsockets.h"
#include "steam/steamnetworkingtypes.h"
// IWYU pragma: end_exports
THIRD_PARTY_INCLUDES_END

