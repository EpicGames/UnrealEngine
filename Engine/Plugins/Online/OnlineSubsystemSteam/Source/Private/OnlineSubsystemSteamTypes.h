// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemNames.h"
#include "OnlineSubsystemSteamPrivate.h" // IWYU pragma: keep
#include "IPAddress.h"
#include "OnlineSubsystemSteamPackage.h"

/** Possible session states */
namespace ESteamSession
{
	enum Type
	{
		/** Session is undefined */
		None,
		/** Session managed as a lobby on backend */
		LobbySession,
		/** Session managed by the server list */
		AdvertisedSessionHost,
		/** Session client of a game server session */
		AdvertisedSessionClient,
		/** Session managed by LAN beacon */
		LANSession,
	};

	/** @return the stringified version of the enum passed in */
	inline const TCHAR* ToString(ESteamSession::Type SessionType)
	{
		switch (SessionType)
		{
		case None:
			{
				return TEXT("Session undefined");
			}
		case LobbySession:
			{
				return TEXT("Lobby session");
			}
		case AdvertisedSessionHost:
			{
				return TEXT("Advertised Session Host");
			}
		case AdvertisedSessionClient:
			{
				return TEXT("Advertised Session Client");
			}
		case LANSession:
			{
				return TEXT("LAN Session");
			}
		}
		return TEXT("");
	}
}

using FUniqueNetIdSteamRef = TSharedRef<const class FUniqueNetIdSteam>;
using FUniqueNetIdSteamPtr = TSharedPtr<const class FUniqueNetIdSteam>;

/**
 * Steam specific implementation of the unique net id
 */
class FUniqueNetIdSteam :
	public FUniqueNetId
{
PACKAGE_SCOPE:
	/** Holds the net id for a player */
	uint64 UniqueNetId;

public:
	template<typename... TArgs>
	static FUniqueNetIdSteamRef Create(TArgs&&... Args)
	{
		return MakeShareable(new FUniqueNetIdSteam(Forward<TArgs>(Args)...));
	}

	static const FUniqueNetIdSteam& Cast(const FUniqueNetId& NetId)
	{
		return *static_cast<const FUniqueNetIdSteam*>(&NetId);
	}

	FUniqueNetIdSteamRef AsShared() const
	{
		return StaticCastSharedRef<const FUniqueNetIdSteam>(FUniqueNetId::AsShared());
	}

	virtual FName GetType() const override
	{
		return STEAM_SUBSYSTEM;
	}

	/**
	 * Get the raw byte representation of this net id
	 * This data is platform dependent and shouldn't be manipulated directly
	 *
	 * @return byte array of size GetSize()
	 */
	virtual const uint8* GetBytes() const override
	{
		return (uint8*)&UniqueNetId;
	}

	/**
	 * Get the size of the id
	 *
	 * @return size in bytes of the id representation
	 */
	virtual int32 GetSize() const override
	{
		return sizeof(uint64);
	}

	/**
	 * Check the validity of the id 
	 *
	 * @return true if this is a well formed ID, false otherwise
	 */
	virtual bool IsValid() const override
	{
		return UniqueNetId != 0 && CSteamID(UniqueNetId).IsValid();
	}

	/** 
	 * Platform specific conversion to string representation of data
	 *
	 * @return data in string form 
	 */
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("%llu"), UniqueNetId);
	}

	/**
	 * Get a human readable representation of the net id
	 * Shouldn't be used for anything other than logging/debugging
	 *
	 * @return id in string form 
	 */
	virtual FString ToDebugString() const override
	{
		const CSteamID SteamID(UniqueNetId);

		const FString HexStr = FString::Printf(TEXT("[0x%llX]"), UniqueNetId);
		if (SteamID.IsLobby())
		{
			return TEXT("Lobby") + OSS_UNIQUEID_REDACT(*this, HexStr);
		}
		else if (SteamID.BAnonGameServerAccount())
		{
			return TEXT("Server") + OSS_UNIQUEID_REDACT(*this, HexStr);
		}
		else if (SteamID.IsValid())
		{
			const FString NickName(SteamFriends() ? UTF8_TO_TCHAR(SteamFriends()->GetFriendPersonaName(UniqueNetId)) : TEXT("UNKNOWN"));
			return FString::Printf(TEXT("%s %s"), *NickName, *OSS_UNIQUEID_REDACT(*this, HexStr));
		}
		else
		{
			return TEXT("INVALID") + OSS_UNIQUEID_REDACT(*this, HexStr);
		}
	}

	static FString ToDebugString(CSteamID SteamID)
	{
		// Safe to use a local variable in this specific case, nothing will call AsShared on it.
		return FUniqueNetIdSteam(SteamID).ToDebugString();
	}

	virtual uint32 GetTypeHash() const override
	{
		return ::GetTypeHash(UniqueNetId);
	}

	/** global static instance of invalid (zero) id */
	static const FUniqueNetIdSteamRef& EmptyId()
	{
		static const FUniqueNetIdSteamRef EmptyId(Create());
		return EmptyId;
	}

	/** Convenience cast to CSteamID */
	operator CSteamID()
	{
		return UniqueNetId;
	}

	/** Convenience cast to CSteamID */
 	operator const CSteamID() const
 	{
 		return UniqueNetId;
 	}

	/** Convenience cast to CSteamID pointer */
	operator CSteamID*()
	{
		return (CSteamID*)&UniqueNetId;
	}

	/** Convenience cast to CSteamID pointer */
	operator const CSteamID*() const
	{
		return (const CSteamID*)&UniqueNetId;
	}

private:
	/** Hidden on purpose */
	FUniqueNetIdSteam() :
		UniqueNetId(0)
	{
	}

	/**
	 * Copy Constructor
	 *
	 * @param Src the id to copy
	 */
	explicit FUniqueNetIdSteam(const FUniqueNetIdSteam& Src) :
		UniqueNetId(Src.UniqueNetId)
	{
	}

	/**
	 * Constructs this object with the specified net id
	 *
	 * @param InUniqueNetId the id to set ours to
	 */
	explicit FUniqueNetIdSteam(uint64 InUniqueNetId) :
		UniqueNetId(InUniqueNetId)
	{
	}

	/**
	 * Constructs this object with the steam id
	 *
	 * @param InUniqueNetId the id to set ours to
	 */
	explicit FUniqueNetIdSteam(CSteamID InSteamId) :
		UniqueNetId(InSteamId.ConvertToUint64())
	{
	}

	/**
	 * Constructs this object with the specified net id
	 *
	 * @param String textual representation of an id
	 */
	explicit FUniqueNetIdSteam(const FString& Str) :
		UniqueNetId(FCString::Atoi64(*Str))
	{
	}

	/**
	 * Constructs this object with the specified net id
	 *
	 * @param InUniqueNetId the id to set ours to (assumed to be FUniqueNetIdSteam in fact)
	 */
	explicit FUniqueNetIdSteam(const FUniqueNetId& InUniqueNetId) :
		UniqueNetId(*(uint64*)InUniqueNetId.GetBytes())
	{
	}
};

/** Data regarding preferred session connection methods */
enum class FSteamConnectionMethod : int8
{
	None=0,
	Direct,
	P2P,
	PartnerHosted
};

/** Functionality for converting a connection method type between formats */
FString LexToString(const FSteamConnectionMethod Method);
FSteamConnectionMethod ToConnectionMethod(const FString& InString);

/** 
 * Implementation of session information
 */
class FOnlineSessionInfoSteam : public FOnlineSessionInfo
{
protected:
	
	/** Hidden on purpose */
	FOnlineSessionInfoSteam(const FOnlineSessionInfoSteam& Src) = delete;
	FOnlineSessionInfoSteam& operator=(const FOnlineSessionInfoSteam& Src) = delete;

PACKAGE_SCOPE:

	/** Constructor for LAN sessions */
	FOnlineSessionInfoSteam(ESteamSession::Type SessionType = ESteamSession::None);

	/** Constructor for sessions that represent a Steam lobby or an advertised server session */
	FOnlineSessionInfoSteam(ESteamSession::Type SessionType, const FUniqueNetIdSteam& InSessionId);

	/** 
	 * Initialize a Steam session info with the address of this machine
	 */
	void Init();

	/** 
	 * Initialize a Steam session info with the address of this machine
	 */
	void InitLAN();

	/** Type of session this is, affects interpretation of id below */
	ESteamSession::Type SessionType;
	/** The ip & port that the host is listening on (valid for LAN/GameServer) */
	TSharedPtr<class FInternetAddr> HostAddr;
	/** The Steam P2P address that the host is listening on (valid for GameServer/Lobby) */
	TSharedPtr<class FInternetAddr> SteamP2PAddr;
	/** Steam Lobby Id or Gameserver Id if applicable */
	FUniqueNetIdSteamRef SessionId;
	/** How this session should be connected to */
	FSteamConnectionMethod ConnectionMethod;

public:

	virtual ~FOnlineSessionInfoSteam() {}

	/**
	 *	Comparison operator
	 */
 	bool operator==(const FOnlineSessionInfoSteam& Other) const
 	{
 		return false;
 	}

	virtual const uint8* GetBytes() const override
	{
		return nullptr;
	}

	virtual int32 GetSize() const override
	{
		return sizeof(uint64) + 
			sizeof(ESteamSession::Type) +
			sizeof(TSharedPtr<class FInternetAddr>) +
			sizeof(TSharedPtr<class FInternetAddr>) + 
			sizeof(FUniqueNetIdSteam) +
			sizeof(FSteamConnectionMethod);
	}

	virtual bool IsValid() const override
	{
		switch (SessionType)
		{
		case ESteamSession::LobbySession:
			return SteamP2PAddr.IsValid() && SteamP2PAddr->IsValid() && SessionId->IsValid();
		case ESteamSession::AdvertisedSessionHost:
		case ESteamSession::AdvertisedSessionClient:
			return ((SteamP2PAddr.IsValid() && SteamP2PAddr->IsValid()) || (HostAddr.IsValid() && HostAddr->IsValid())) && SessionId->IsValid();
		case ESteamSession::LANSession:
		default:
			// LAN case
			return HostAddr.IsValid() && HostAddr->IsValid();
		}
	}

	virtual FString ToString() const override
	{
		return SessionId->ToString();
	}

	virtual FString ToDebugString() const override
	{
		return FString::Printf(TEXT("HostIP: %s SteamP2P: %s Type: %s SessionId: %s"), 
			HostAddr.IsValid() ? *HostAddr->ToString(true) : TEXT("INVALID"), 
			SteamP2PAddr.IsValid() ? *SteamP2PAddr->ToString(true) : TEXT("INVALID"),
			ESteamSession::ToString(SessionType), *SessionId->ToDebugString());
	}

	virtual const FUniqueNetId& GetSessionId() const override
	{
		return *SessionId;
	}
};

/**
 * Steam specific implementation of a shared file handle
 */
class FSharedContentHandleSteam :
	public FSharedContentHandle
{

	/** Holds the handle to the shared content */
	const UGCHandle_t SharedContentHandle;

	/** Hidden on purpose */
	FSharedContentHandleSteam() = delete;

public:

	/**
	 * Constructs this object with the specified shared content id
	 *
	 * @param InUniqueNetId the id to set ours to
	 */
	FSharedContentHandleSteam(UGCHandle_t InSharedContentHandle) :
		SharedContentHandle(InSharedContentHandle)
	{
	}

	/**
	 *	Comparison operator
	 */
 	bool operator==(const FSharedContentHandleSteam& Other) const
 	{
 		return (SharedContentHandle == Other.SharedContentHandle);
 	}

	/** 
	 * Get the raw byte representation of this shared content handle
	 * This data is platform dependent and shouldn't be manipulated directly
	 *
	 * @return byte array of size GetSize()
	 */
	virtual const uint8* GetBytes() const override
	{
		return (uint8*)&SharedContentHandle;
	}

	/** 
	 * Get the size of this shared content handle
	 *
	 * @return size in bytes of the id representation
	 */
	virtual int32 GetSize() const override
	{
		return sizeof(UGCHandle_t);
	}

	/** 
	 * Check the validity of this shared content handle
	 *
	 * @return true if this is a well formed ID, false otherwise
	 */
	virtual bool IsValid() const override
	{
		return SharedContentHandle != k_UGCHandleInvalid;
	}

	/** 
	 * Platform specific conversion to string representation of data
	 *
	 * @return data in string form 
	 */
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("%lld"), SharedContentHandle);
	}

	/** 
	 * Get a human readable representation of this shared content handle
	 * Shouldn't be used for anything other than logging/debugging
	 *
	 * @return handle in string form 
	 */
	virtual FString ToDebugString() const override
	{
		return FString::Printf(TEXT("0x%llX"), SharedContentHandle);
	}
};

/** Holds the data used in downloading a file asynchronously from the online service */
struct FCloudFileSteam : public FCloudFile
{
	/** The name of the file as requested */
	const FSharedContentHandleSteam SharedHandle;

	/** Constructors */
	FCloudFileSteam() :
		SharedHandle(k_UGCHandleInvalid)
	{
	}

	FCloudFileSteam(const FSharedContentHandleSteam& InSharedHandle) :
		SharedHandle(InSharedHandle)
	{
	}

	~FCloudFileSteam() {}
};

/** 
 * Record containing the state of cloud files for a given user
 * as requested from <Enumerate/Read/Write/Delete><User/Shared>Files
 * Does not necessarily represent the full state of all files for a given user
 *
 * This is an ASYNC data structure, make sure proper locks are in place before manipulating
 */
struct FSteamUserCloudData
{
	/** File metadata */
    TArray<struct FCloudFileHeader> CloudMetadata;
	/** File cache */
    TArray<struct FCloudFile> CloudFileData;

public:
	/** Owning user for these files */
    const FUniqueNetIdSteamRef UserId;

    /** Constructors */
    FSteamUserCloudData(const FUniqueNetIdSteam& InUserId) :
		UserId(InUserId.AsShared())
	{
	}

	~FSteamUserCloudData()
	{
		ClearMetadata();
		ClearFiles();
	}

	/** 
	 * Clear out all cached data for a given user
	 * Doesn't touch metadata
	 * @return true if successful, false otherwise
	 */
	bool ClearFiles()
	{
		// Delete file contents
		for (int32 FileIdx = 0; FileIdx < CloudFileData.Num(); FileIdx++)
		{
			CloudFileData[FileIdx].Data.Empty();
		}

		// No async files being handled, so empty them all
		CloudFileData.Empty();
		return true;
	}

	/** 
	 * Clear out cached data of a given file for a given user
	 * Doesn't touch metadata
	 * @param FileName file data to clear
	 * @return true if file was cleared, false otherwise
	 */
	bool ClearFileData(const FString& FileName)
	{
		int32 FoundIndex = INDEX_NONE;
		for (int32 FileIdx = 0; FileIdx < CloudFileData.Num(); FileIdx++)
		{
			FCloudFile* UserFileData = &CloudFileData[FileIdx];
			if (UserFileData->FileName == FileName)
			{
				// If there is an async task outstanding, fail to empty
				if (UserFileData->AsyncState == EOnlineAsyncTaskState::InProgress)
				{
					return false;
				}

				UserFileData->Data.Empty();
				FoundIndex = FileIdx;
				break;
			}
		}

		if (FoundIndex != INDEX_NONE)
		{
			CloudFileData.RemoveAtSwap(FoundIndex);
		}
		return true;
	}

	/** 
	 * Clear out all file metadata 
	 * Doesn't touch actual cached file data contents
	 */
	void ClearMetadata()
	{
		CloudMetadata.Empty();
	}

	/** 
	 * Clear out metadata for a given file 
	 * Doesn't touch actual cached file data contents
	 * @param FileName file metadata to clear
	 */
	void ClearMetadata(const FString& FileName)
	{
		int32 FoundIndex = INDEX_NONE;
		for (int32 FileIdx = 0; FileIdx < CloudMetadata.Num(); FileIdx++)
		{
			FCloudFileHeader* UserFileData = &CloudMetadata[FileIdx];
			if (UserFileData &&
				UserFileData->FileName == FileName)
			{
				FoundIndex = FileIdx;
				break;
			}
		}

		if (FoundIndex != INDEX_NONE)
		{
			CloudMetadata.RemoveAtSwap(FoundIndex);
		}
	}

	/** 
     * Get the metadata related to a given user's file on Steam
     * This information is only available after calling EnumerateUserFiles
     *
     * @param FileName the file to get metadata about
	 * @param bCreateIfMissing create an entry if not found
     * @return the struct with the metadata about the requested file, NULL otherwise
     */
	struct FCloudFileHeader* GetFileMetadata(const FString& FileName, bool bCreateIfMissing=false)
	{
		if (FileName.Len() > 0)
		{
			for (int32 FileIdx = 0; FileIdx < CloudMetadata.Num(); FileIdx++)
			{
				FCloudFileHeader* UserFileData = &CloudMetadata[FileIdx];
				if (UserFileData &&
					UserFileData->FileName == FileName)
				{
					return UserFileData;
				}
			}

			if (bCreateIfMissing)
			{
				return &CloudMetadata.Emplace_GetRef(FileName, FileName, 0);
			}
		}

		return NULL;
	}

	/**
	 * Get physical/logical file information for a given user's cloud file
     *
	 * @param FileName the file to search for
	 * @param bCreateIfMissing create an entry if not found
	 * @return the file details, NULL otherwise
	 */
	struct FCloudFile* GetFileData(const FString& FileName, bool bCreateIfMissing=false)
	{
		if (FileName.Len() > 0)
		{
			for (int32 FileIdx = 0; FileIdx < CloudFileData.Num(); FileIdx++)
			{
				FCloudFile* UserFileData = &CloudFileData[FileIdx];
				if (UserFileData &&
					UserFileData->FileName == FileName)
				{
					return UserFileData;
				}
			}

			if (bCreateIfMissing)
			{
				return &CloudFileData.Emplace_GetRef(FileName);
			}
		}

		return nullptr;
	}
};

/** 
 * Basic leaderboard representation
 */
class FLeaderboardMetadataSteam
{
	FLeaderboardMetadataSteam() {}

public:

	/** Name of leaderboard, matches Steam backend */
	FString LeaderboardName;
	/** Sort Method */
	ELeaderboardSort::Type SortMethod;
	/** Display Type */
	ELeaderboardFormat::Type DisplayFormat;
	/** Number of entries on leaderboard */
	int32 TotalLeaderboardRows;
	/** Handle to leaderboard */
    SteamLeaderboard_t LeaderboardHandle;
	/** State of the leaderboard handle download */
	EOnlineAsyncTaskState::Type AsyncState;

	FLeaderboardMetadataSteam(const FString& InLeaderboardName, ELeaderboardSort::Type InSortMethod, ELeaderboardFormat::Type InDisplayFormat) :
		LeaderboardName(InLeaderboardName),
		SortMethod(InSortMethod),
		DisplayFormat(InDisplayFormat),
		TotalLeaderboardRows(0),
		LeaderboardHandle(-1),
		AsyncState(EOnlineAsyncTaskState::NotStarted)
	{
	}

	FLeaderboardMetadataSteam(const FString& InLeaderboardName) :
		LeaderboardName(InLeaderboardName),
		SortMethod(ELeaderboardSort::None),
		DisplayFormat(ELeaderboardFormat::Number),
		TotalLeaderboardRows(0),
		LeaderboardHandle(-1),
		AsyncState(EOnlineAsyncTaskState::NotStarted)
	{
	}
};
