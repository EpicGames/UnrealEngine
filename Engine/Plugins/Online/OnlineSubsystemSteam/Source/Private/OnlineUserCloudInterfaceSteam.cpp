// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineUserCloudInterfaceSteam.h"
#include "Misc/ScopeLock.h"
#include "OnlineSubsystemSteam.h"

FString FOnlineAsyncTaskSteamEnumerateUserFiles::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncTaskSteamEnumerateUserFiles bWasSuccessful:%d UserId:%s"),
									WasSuccessful(), *UserId->ToDebugString());
}

void FOnlineAsyncTaskSteamEnumerateUserFiles::Tick()
{
	bIsComplete = true;
	bWasSuccessful = false;

	if (SteamRemoteStorage())
	{
		if (SteamUser()->BLoggedOn() && SteamUser()->GetSteamID() == *UserId)
		{
			//SteamSubsystem->GetUserCloudInterface()->DumpCloudState(UserId);

			FScopeLock ScopeLock(&Subsystem->UserCloudDataLock);

			// Get or create the user metadata entry and empty it
			FSteamUserCloudData* UserMetadata = Subsystem->GetUserCloudEntry(*UserId);

			UserMetadata->CloudMetadata.Empty();

			// Fill in the metadata entries
			const int32 FileCount = (int32) SteamRemoteStorage()->GetFileCount();
			for (int32 FileIdx = 0; FileIdx < FileCount; FileIdx++)
			{
				int32 FileSize = 0;
				const char *FileName = SteamRemoteStorage()->GetFileNameAndSize(FileIdx, &FileSize);
				new (UserMetadata->CloudMetadata) FCloudFileHeader(UTF8_TO_TCHAR(FileName), UTF8_TO_TCHAR(FileName), int32(FileSize));

				//SteamSubsystem->GetUserCloudInterface()->DumpCloudFileState(UserId, FileName);
			}

			bWasSuccessful = true;
		}
		else
		{
			UE_LOG_ONLINE_CLOUD(Warning, TEXT("Can only enumerate cloud files for logged in user."));
		}
	}
	else
	{
		UE_LOG_ONLINE_CLOUD(Warning, TEXT("Steam remote storage API disabled."));
	}
}

void FOnlineAsyncTaskSteamEnumerateUserFiles::TriggerDelegates()
{ 
	FOnlineAsyncTaskSteam::TriggerDelegates();

	IOnlineUserCloudPtr UserCloudInterface = Subsystem->GetUserCloudInterface();
	UserCloudInterface->TriggerOnEnumerateUserFilesCompleteDelegates(bWasSuccessful, *UserId);
}

FString FOnlineAsyncTaskSteamReadUserFile::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncTaskSteamReadUserFile bWasSuccessful:%d UserId:%s FileName:%s"),
									WasSuccessful(), *UserId->ToDebugString(), *FileName);
}

void FOnlineAsyncTaskSteamReadUserFile::Tick()
{
	// Going to be complete no matter what
	bIsComplete = true;

	if (SteamRemoteStorage() && FileName.Len() > 0)
	{
		if (SteamUser()->BLoggedOn() && SteamUser()->GetSteamID() == *UserId)
		{
			// Currently don't support greater than 1 chunk
			const int32 FileSize = SteamRemoteStorage()->GetFileSize(TCHAR_TO_UTF8(*FileName));
			if (FileSize >= 0 && FileSize <= k_unMaxCloudFileChunkSize)
			{
				FScopeLock ScopeLock(&Subsystem->UserCloudDataLock);
				// Create or get the current entry for this file
				FSteamUserCloudData* UserCloud = Subsystem->GetUserCloudEntry(*UserId);
				if (UserCloud)
				{
					FCloudFile* UserCloudFile = UserCloud->GetFileData(FileName, true);
					check(UserCloudFile);

					// Allocate and read in the file
					UserCloudFile->Data.Empty(FileSize);
					UserCloudFile->Data.AddUninitialized(FileSize);
					if (SteamRemoteStorage()->FileRead(TCHAR_TO_UTF8(*FileName), UserCloudFile->Data.GetData(), FileSize) == FileSize)
					{
						bWasSuccessful = true;
					}
					else
					{
						UserCloudFile->Data.Empty();
					}
				}
			}
			else
			{
				UE_LOG_ONLINE_CLOUD(Warning, TEXT("Requested file %s has invalid size %d."), *FileName, FileSize);
			}
		}	
		else
		{
			UE_LOG_ONLINE_CLOUD(Warning, TEXT("Can only read cloud files for logged in user."));
		}
	}
	else
	{
		UE_LOG_ONLINE_CLOUD(Warning, TEXT("Steam remote storage API disabled."));
	}

	{
		FScopeLock ScopeLock(&Subsystem->UserCloudDataLock);
		FSteamUserCloudData* UserCloud = Subsystem->GetUserCloudEntry(*UserId);
		if (UserCloud)
		{
			FCloudFile* UserCloudFileData = UserCloud->GetFileData(FileName);
			if (UserCloudFileData)
			{
				UserCloudFileData->AsyncState = bWasSuccessful ? EOnlineAsyncTaskState::Done : EOnlineAsyncTaskState::Failed;
			}
		}
	}
}

void FOnlineAsyncTaskSteamReadUserFile::TriggerDelegates()
{ 
	FOnlineAsyncTaskSteam::TriggerDelegates();

	IOnlineUserCloudPtr UserCloudInterface = Subsystem->GetUserCloudInterface();
	UserCloudInterface->TriggerOnReadUserFileCompleteDelegates(bWasSuccessful, *UserId, FileName);
}

bool FOnlineAsyncTaskSteamWriteUserFile::WriteUserFile(const FUniqueNetId& InUserId, const FString& InFileToWrite, const TArray<uint8>& InContents)
{
	bool bSuccess = false;
	if (InFileToWrite.Len() > 0 && InContents.Num() > 0)
	{
		if (SteamRemoteStorage() && FileName.Len() > 0)
		{
			CSteamID SteamId(*(uint64*)InUserId.GetBytes());
			if (SteamUser()->BLoggedOn() && SteamUser()->GetSteamID() == SteamId)
			{
				// Currently don't support greater than 1 chunk
				if (InContents.Num() < k_unMaxCloudFileChunkSize)
				{
					if (SteamRemoteStorage()->FileWrite(TCHAR_TO_UTF8(*InFileToWrite), InContents.GetData(), InContents.Num()))
					{
						FScopeLock ScopeLock(&Subsystem->UserCloudDataLock);
						FSteamUserCloudData* UserCloud = Subsystem->GetUserCloudEntry(InUserId);
						if (UserCloud)
						{
							// Update the metadata table to reflect this write (might be new entry)
							FCloudFileHeader* UserCloudFileMetadata = UserCloud->GetFileMetadata(InFileToWrite, true);
							check(UserCloudFileMetadata);

							UserCloudFileMetadata->FileSize = SteamRemoteStorage()->GetFileSize(TCHAR_TO_UTF8(*InFileToWrite));
							UserCloudFileMetadata->Hash = FString(TEXT("0"));

							// Update the file table to reflect this write
							FCloudFile* UserCloudFileData = UserCloud->GetFileData(InFileToWrite, true);
							check(UserCloudFileData);

							UserCloudFileData->Data = InContents;
							bSuccess = true;
						}
					}
					else
					{
						UE_LOG_ONLINE_CLOUD(Warning, TEXT("Failed to write file to Steam cloud \"%s\"."), *InFileToWrite);
					}
				}
				else
				{
					UE_LOG_ONLINE_CLOUD(Warning, TEXT("File too large %d to write to Steam cloud."), InContents.Num());
				}
			}
			else
			{
				UE_LOG_ONLINE_CLOUD(Warning, TEXT("Can only write cloud files for logged in user."));
			}
		}
		else
		{
			UE_LOG_ONLINE_CLOUD(Warning, TEXT("Steam remote storage API disabled."));
		}
	}

	{
		FScopeLock ScopeLock(&Subsystem->UserCloudDataLock);
		FSteamUserCloudData* UserCloud = Subsystem->GetUserCloudEntry(InUserId);
		if (UserCloud)
		{
			FCloudFile* UserCloudFileData = UserCloud->GetFileData(InFileToWrite, true);
			check(UserCloudFileData);
			UserCloudFileData->AsyncState = bSuccess ? EOnlineAsyncTaskState::Done : EOnlineAsyncTaskState::Failed;
		}
	}

	//SteamSubsystem->GetUserCloudInterface()->DumpCloudFileState(InUserId, InFileToWrite);
	return bSuccess;
}

FString FOnlineAsyncTaskSteamWriteUserFile::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncTaskSteamWriteUserFile bWasSuccessful:%d UserId:%s FileName:%s"),
									WasSuccessful(), *UserId->ToDebugString(), *FileName);
}

void FOnlineAsyncTaskSteamWriteUserFile::Tick()
{	
	// Going to be complete no matter what
	bIsComplete = true;
	if (WriteUserFile(*UserId, FileName, Contents))
	{
		bWasSuccessful = true;
	}

	// Done with this copy of the data regardless
	Contents.Empty();
}

void FOnlineAsyncTaskSteamWriteUserFile::TriggerDelegates()
{ 
	FOnlineAsyncTaskSteam::TriggerDelegates();

	IOnlineUserCloudPtr UserCloudInterface = Subsystem->GetUserCloudInterface();
	UserCloudInterface->TriggerOnWriteUserFileCompleteDelegates(bWasSuccessful, *UserId, FileName);
}

FOnlineUserCloudSteam::~FOnlineUserCloudSteam()
{
	if (SteamSubsystem)
	{
		SteamSubsystem->ClearUserCloudFiles();
	}
}

bool FOnlineUserCloudSteam::GetFileContents(const FUniqueNetId& UserId, const FString& FileName, TArray<uint8>& FileContents)
{
	FScopeLock ScopeLock(&SteamSubsystem->UserCloudDataLock);
	// Search for the specified file and return the raw data
	FSteamUserCloudData* UserCloudData = SteamSubsystem->GetUserCloudEntry(UserId);
	if (UserCloudData)
	{
		FCloudFile* SteamCloudFile = UserCloudData->GetFileData(FileName);
		if (SteamCloudFile && SteamCloudFile->AsyncState == EOnlineAsyncTaskState::Done && SteamCloudFile->Data.Num() > 0)
		{
			FileContents = SteamCloudFile->Data;
			return true;
		}
	}
	
	return false;
}

bool FOnlineUserCloudSteam::ClearFiles(const FUniqueNetId& UserId)
{
	FScopeLock ScopeLock(&SteamSubsystem->UserCloudDataLock);
	// Search for the specified file
	FSteamUserCloudData* UserCloudData = SteamSubsystem->GetUserCloudEntry(UserId);
	if (UserCloudData)
	{
		return UserCloudData->ClearFiles();
	}

	return true;
}

bool FOnlineUserCloudSteam::ClearFile(const FUniqueNetId& UserId, const FString& FileName)
{
	FScopeLock ScopeLock(&SteamSubsystem->UserCloudDataLock);
	// Search for the specified file
	FSteamUserCloudData* UserCloudData = SteamSubsystem->GetUserCloudEntry(UserId);
	if (UserCloudData)
	{
		return UserCloudData->ClearFileData(FileName);
	}

	return true;
}

void FOnlineUserCloudSteam::EnumerateUserFiles(const FUniqueNetId& UserId)
{
	SteamSubsystem->QueueAsyncTask(new FOnlineAsyncTaskSteamEnumerateUserFiles(SteamSubsystem, FUniqueNetIdSteam::Cast(UserId)));
}

void FOnlineUserCloudSteam::GetUserFileList(const FUniqueNetId& UserId, TArray<FCloudFileHeader>& UserFiles)
{
	FScopeLock ScopeLock(&SteamSubsystem->UserCloudDataLock);
	FSteamUserCloudData* UserMetadata = SteamSubsystem->GetUserCloudEntry(UserId);  
	UserFiles = UserMetadata->CloudMetadata;
}

bool FOnlineUserCloudSteam::ReadUserFile(const FUniqueNetId& UserId, const FString& FileName)
{
	FScopeLock ScopeLock(&SteamSubsystem->UserCloudDataLock);
	// Create or get the current entry for this file
	FSteamUserCloudData* UserCloud = SteamSubsystem->GetUserCloudEntry(UserId);
	if (UserCloud && FileName.Len() > 0)
	{
		FCloudFile* UserCloudFile = UserCloud->GetFileData(FileName, true);
		UserCloudFile->AsyncState = EOnlineAsyncTaskState::InProgress;
		SteamSubsystem->QueueAsyncTask(new FOnlineAsyncTaskSteamReadUserFile(SteamSubsystem, FUniqueNetIdSteam::Cast(UserId), FileName));
		return true;
	}
	
	return false;
}

bool FOnlineUserCloudSteam::WriteUserFile(const FUniqueNetId& UserId, const FString& FileName, TArray<uint8>& FileContents, bool bCompressBeforeUpload)
{
	FScopeLock ScopeLock(&SteamSubsystem->UserCloudDataLock);
	// Create or get the current entry for this file
	FSteamUserCloudData* UserCloud = SteamSubsystem->GetUserCloudEntry(UserId);
	if (UserCloud && FileName.Len() > 0)
	{
		FCloudFile* UserCloudFile = UserCloud->GetFileData(FileName, true);
		UserCloudFile->AsyncState = EOnlineAsyncTaskState::InProgress;
		SteamSubsystem->QueueAsyncTask(new FOnlineAsyncTaskSteamWriteUserFile(SteamSubsystem, FUniqueNetIdSteam::Cast(UserId), FileName, FileContents));
		return true;
	}

	return false;
}

void FOnlineUserCloudSteam::CancelWriteUserFile(const FUniqueNetId& UserId, const FString& FileName)
{
	// Not implemented
}


bool FOnlineUserCloudSteam::DeleteUserFile(const FUniqueNetId& UserId, const FString& FileName, bool bShouldCloudDelete, bool bShouldLocallyDelete)
{
	SteamSubsystem->QueueAsyncTask(new FOnlineAsyncTaskSteamDeleteUserFile(SteamSubsystem, FUniqueNetIdSteam::Cast(UserId), FileName, bShouldCloudDelete, bShouldLocallyDelete));
	return true;
}

bool FOnlineUserCloudSteam::RequestUsageInfo(const FUniqueNetId& UserId)
{
	// Not implemented
	return false;
}

void FOnlineUserCloudSteam::DumpCloudState(const FUniqueNetId& UserId)
{
	uint64 TotalBytes, TotalAvailable;
	if (SteamRemoteStorage()->GetQuota(&TotalBytes, &TotalAvailable) == false)
	{
		TotalBytes = TotalAvailable = 0;
	}

	UE_LOG_ONLINE_CLOUD(Verbose, TEXT("Steam Disk Quota: %" UINT64_FMT " / %" UINT64_FMT), TotalAvailable, TotalBytes);
	UE_LOG_ONLINE_CLOUD(Verbose, TEXT("Game does %shave cloud storage enabled."), SteamRemoteStorage()->IsCloudEnabledForApp() ? TEXT("") : TEXT("NOT "));
	UE_LOG_ONLINE_CLOUD(Verbose, TEXT("User does %shave cloud storage enabled."), SteamRemoteStorage()->IsCloudEnabledForAccount() ? TEXT("") : TEXT("NOT "));
}

void FOnlineUserCloudSteam::DumpCloudFileState(const FUniqueNetId& UserId, const FString& FileName)
{
	if (FileName.Len() > 0)
	{
		UE_LOG_ONLINE_CLOUD(Log, TEXT("Cloud File State file %s:"), *FileName);
		{
			FScopeLock ScopeLock(&SteamSubsystem->UserCloudDataLock);
			FSteamUserCloudData* UserCloud = SteamSubsystem->GetUserCloudEntry(UserId);
			FCloudFileHeader* FileMetadata = UserCloud->GetFileMetadata(FileName);
			if (FileMetadata)
			{
				UE_LOG_ONLINE_CLOUD(Log, TEXT("\tMeta: FileName:%s DLName:%s FileSize:%d Hash:%s"), *FileMetadata->FileName, *FileMetadata->DLName, FileMetadata->FileSize, *FileMetadata->Hash);
			}
			else
			{
				UE_LOG_ONLINE_CLOUD(Log, TEXT("\tNo metadata found!"));
			}

			FCloudFile* FileData = UserCloud->GetFileData(FileName);
			if (FileData)
			{
				UE_LOG_ONLINE_CLOUD(Log, TEXT("\tFileCache: FileName:%s State:%s CacheSize:%d"), *FileData->FileName, EOnlineAsyncTaskState::ToString(FileData->AsyncState), FileData->Data.Num());
			}
			else
			{
				UE_LOG_ONLINE_CLOUD(Log, TEXT("\tNo cache entry found!"));
			}
		}

		int32 FileSize = SteamRemoteStorage()->GetFileSize(TCHAR_TO_UTF8(*FileName));

		UE_LOG_ONLINE_CLOUD(Log, TEXT("\tSteam: FileName:%s Size:%d Exists:%s Persistent:%s"),
			*FileName, FileSize, 
			SteamRemoteStorage()->FileExists(TCHAR_TO_UTF8(*FileName)) ? TEXT("Y") : TEXT("N"),
			SteamRemoteStorage()->FilePersisted(TCHAR_TO_UTF8(*FileName)) ? TEXT("Y") : TEXT("N"));
	}
}

FString FOnlineAsyncTaskSteamDeleteUserFile::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncTaskSteamDeleteUserFile bWasSuccessful:%d UserId:%s FileName:%s"),
								WasSuccessful(), *UserId->ToDebugString(), *FileName);
}

void FOnlineAsyncTaskSteamDeleteUserFile::Tick()
{
	bWasSuccessful = false;

	if (SteamRemoteStorage() && FileName.Len() > 0)
	{
		if (SteamUser()->BLoggedOn() && SteamUser()->GetSteamID() == *UserId)
		{
			bool bCloudDeleteSuccess = true;
			if (bShouldCloudDelete)
			{
				// Remove the cloud flag, the file remains safely available on the local machine
				bCloudDeleteSuccess = SteamRemoteStorage()->FileForget(TCHAR_TO_UTF8(*FileName));
			}

			bool bLocalDeleteSuccess = true;
			if (bShouldLocallyDelete)
			{
				bLocalDeleteSuccess = false;
				// Only clear the tables if we're permanently deleting the file
				// Need to make sure nothing async is happening	first (this is a formality as nothing in Steam actually is)
				IOnlineUserCloudPtr UserCloud = Subsystem->GetUserCloudInterface();
				if (UserCloud->ClearFile(*UserId, FileName))
				{
					// Permanent delete
					bLocalDeleteSuccess = SteamRemoteStorage()->FileDelete(TCHAR_TO_UTF8(*FileName));
					Subsystem->ClearUserCloudMetadata(*UserId, FileName);
				}
			}

			bWasSuccessful = bCloudDeleteSuccess && bLocalDeleteSuccess;
		}	
		else
		{
			UE_LOG_ONLINE_CLOUD(Warning, TEXT("Can only delete cloud files for logged in user."));
		}
	}
	else
	{
		UE_LOG_ONLINE_CLOUD(Warning, TEXT("Steam remote storage API disabled."));
	}

	bIsComplete = true;
}

void FOnlineAsyncTaskSteamDeleteUserFile::TriggerDelegates()
{ 
	FOnlineAsyncTaskSteam::TriggerDelegates();

	IOnlineUserCloudPtr UserCloudInterface = Subsystem->GetUserCloudInterface();
	UserCloudInterface->TriggerOnDeleteUserFileCompleteDelegates(bWasSuccessful, *UserId, FileName);
}
