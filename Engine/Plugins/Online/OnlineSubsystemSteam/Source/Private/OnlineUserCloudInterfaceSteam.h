// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
#include "Interfaces/OnlineUserCloudInterface.h"
#include "OnlineSubsystemSteamTypes.h"
#include "OnlineAsyncTaskManagerSteam.h"

/** 
 *  Async task for enumerating all cloud files for a given user
 */
class FOnlineAsyncTaskSteamEnumerateUserFiles : public FOnlineAsyncTaskSteam
{
	/** UserId for file enumeration */
	FUniqueNetIdSteamRef UserId;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamEnumerateUserFiles() = delete;

public:
	FOnlineAsyncTaskSteamEnumerateUserFiles(class FOnlineSubsystemSteam* InSubsystem, const FUniqueNetIdSteam& InUserId) :
		FOnlineAsyncTaskSteam(InSubsystem, k_uAPICallInvalid),
		UserId(InUserId.AsShared())
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override;

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override;
	
	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override;
};

/** 
 *  Async task for reading into memory a single cloud files for a given user
 */
class FOnlineAsyncTaskSteamReadUserFile : public FOnlineAsyncTaskSteam
{
PACKAGE_SCOPE:

	/** UserId making the request */
	FUniqueNetIdSteamRef UserId;
	/** Filename shared */
	FString FileName;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamReadUserFile() = delete;

public:

	FOnlineAsyncTaskSteamReadUserFile(class FOnlineSubsystemSteam* InSubsystem, const FUniqueNetIdSteam& InUserId, const FString& InFileName) :
		FOnlineAsyncTaskSteam(InSubsystem, k_uAPICallInvalid),
		UserId(InUserId.AsShared()), 
		FileName(InFileName)
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override;

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override;
	
	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override;
};

/** 
 *  Async task for writing a single cloud file to disk for a given user
 */
class FOnlineAsyncTaskSteamWriteUserFile : public FOnlineAsyncTaskSteam
{
PACKAGE_SCOPE:

	/** Copy of the data to write */
	TArray<uint8> Contents;
	/** UserId making the request */
	FUniqueNetIdSteamRef UserId;
	/** File being written */
	FString FileName;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamWriteUserFile() = delete;

	/**
	 * Write the specified user file to the network platform's file store
	 *
	 * @param UserId User owning the storage
	 * @param FileToWrite the name of the file to write
	 * @param FileContents the out buffer to copy the data into
	 *
	 * @return true if the calls starts successfully, false otherwise
	 */
	bool WriteUserFile(const FUniqueNetId& UserId, const FString& FileToWrite, const TArray<uint8>& Contents);

public:

	FOnlineAsyncTaskSteamWriteUserFile(class FOnlineSubsystemSteam* InSubsystem, const FUniqueNetIdSteam& InUserId, const FString& InFileName, const TArray<uint8>& InContents) :
		FOnlineAsyncTaskSteam(InSubsystem, k_uAPICallInvalid),
		Contents(InContents),
		UserId(InUserId.AsShared()), 
		FileName(InFileName)
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override;

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override;
	
	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override;
};

/** 
 *  Async task for deleting a single cloud file for a given user
 */
class FOnlineAsyncTaskSteamDeleteUserFile : public FOnlineAsyncTaskSteam
{
	/** Should the file be deleted from the cloud record */
	bool bShouldCloudDelete;
	/** Should the local copy of the file be deleted */
	bool bShouldLocallyDelete;
	/** UserId making the request */
	FUniqueNetIdSteamRef UserId;
	/** File being deleted */
	FString FileName;

	/** Hidden on purpose */
	FOnlineAsyncTaskSteamDeleteUserFile() = delete;

public:
	FOnlineAsyncTaskSteamDeleteUserFile(class FOnlineSubsystemSteam* InSubsystem, const FUniqueNetIdSteam& InUserId, const FString& InFileName, bool bInShouldCloudDelete, bool bInShouldLocallyDelete) :
		FOnlineAsyncTaskSteam(InSubsystem, k_uAPICallInvalid),
		bShouldCloudDelete(bInShouldCloudDelete),
		bShouldLocallyDelete(bInShouldLocallyDelete),
		UserId(InUserId.AsShared()),
		FileName(InFileName)
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override;

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override;
	
	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override;
};

/**
 * Provides access to per user cloud file storage
 */
class FOnlineUserCloudSteam : public IOnlineUserCloud
{
private:
	/** Reference to the main Steam subsystem */
	class FOnlineSubsystemSteam* SteamSubsystem;

	FOnlineUserCloudSteam() :
		SteamSubsystem(NULL)
	{
	}

PACKAGE_SCOPE:

	FOnlineUserCloudSteam(class FOnlineSubsystemSteam* InSubsystem) :
		SteamSubsystem(InSubsystem)
	{
	}

public:
	
	virtual ~FOnlineUserCloudSteam();

	// IOnlineUserCloud
	virtual bool GetFileContents(const FUniqueNetId& UserId, const FString& FileName, TArray<uint8>& FileContents) override;
	virtual bool ClearFiles(const FUniqueNetId& UserId) override;
	virtual bool ClearFile(const FUniqueNetId& UserId, const FString& FileName) override;
	virtual void EnumerateUserFiles(const FUniqueNetId& UserId) override;
	virtual void GetUserFileList(const FUniqueNetId& UserId, TArray<FCloudFileHeader>& UserFiles) override;
	virtual bool ReadUserFile(const FUniqueNetId& UserId, const FString& FileName) override;
	virtual bool WriteUserFile(const FUniqueNetId& UserId, const FString& FileName, TArray<uint8>& FileContents, bool bCompressBeforeUpload = false) override;
	virtual void CancelWriteUserFile(const FUniqueNetId& UserId, const FString& FileName) override;
	virtual bool DeleteUserFile(const FUniqueNetId& UserId, const FString& FileName, bool bShouldCloudDelete, bool bShouldLocallyDelete) override;
	virtual bool RequestUsageInfo(const FUniqueNetId& UserId) override;
	virtual void DumpCloudState(const FUniqueNetId& UserId) override;
	virtual void DumpCloudFileState(const FUniqueNetId& UserId, const FString& FileName) override;

};

typedef TSharedPtr<FOnlineUserCloudSteam, ESPMode::ThreadSafe> FOnlineUserCloudSteamPtr;
