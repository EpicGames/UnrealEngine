// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystemSteamPrivate.h" // IWYU pragma: keep
#include "OnlineSessionSettings.h"

/** Internal Steam session keys for advertising */

/** Well defined lobby/server keys for use with Steam lobby/server data (NOTE: Steam expects UTF8) */
#define STEAMKEY_NUMPUBLICCONNECTIONS "NUMPUBCONN"
#define STEAMKEY_NUMPRIVATECONNECTIONS "NUMPRIVCONN"
#define STEAMKEY_SESSIONFLAGS "SESSIONFLAGS"
#define STEAMKEY_OWNINGUSERID "OWNINGID"
#define STEAMKEY_OWNINGUSERNAME "OWNINGNAME"
#define STEAMKEY_NUMOPENPRIVATECONNECTIONS "NUMOPENPRIVCONN"
#define	STEAMKEY_NUMOPENPUBLICCONNECTIONS "NUMOPENPUBCONN"
#define STEAMKEY_BUILDUNIQUEID "BUILDID"

/** Number of keys above required for valid lobby session */
#define STEAMKEY_NUMREQUIREDLOBBYKEYS 9
#define STEAMKEY_NUMREQUIREDSERVERKEYS 4

/** Optional keys (depends on lobby/advertised server session) */
#define STEAMKEY_HOSTIP "HOSTIP"
#define STEAMKEY_HOSTPORT "HOSTPORT"
#define STEAMKEY_P2PADDR "P2PADDR"
#define STEAMKEY_P2PPORT "P2PPORT"
#define STEAMKEY_P2PPING "P2PPING"
#define STEAMKEY_CONNECTIONMETHOD "CONMETHOD"

/**
 * Converts an engine key and its data type to an appropriate Steam key for use with lobbies/gameservers
 * Encoded as key in the form <keyname>_<datatype> so that it is known client side
 *
 * @param Key online key
 * @param Data data associated with the key
 * @param KeyStr output Steam key for use with Steam lobbies
 * 
 * @return true if successful, false if data type unknown/unsupported
 */
inline bool SessionKeyToSteamKey(const FName Key, const FVariantData& Data, FString& KeyStr)
{
	switch (Data.GetType())
	{
	case EOnlineKeyValuePairDataType::Int32:
		KeyStr = FString::Printf(TEXT("%s_i"), *Key.ToString());
		break;
	case EOnlineKeyValuePairDataType::Int64:
		KeyStr = FString::Printf(TEXT("%s_l"), *Key.ToString());
		break;
	case EOnlineKeyValuePairDataType::Double:	
		KeyStr = FString::Printf(TEXT("%s_d"), *Key.ToString());
		break;
	case EOnlineKeyValuePairDataType::String:
		KeyStr = FString::Printf(TEXT("%s_s"), *Key.ToString());
		break;
	case EOnlineKeyValuePairDataType::Float:
		KeyStr = FString::Printf(TEXT("%s_f"), *Key.ToString());
		break;
	case EOnlineKeyValuePairDataType::Bool:
		KeyStr = FString::Printf(TEXT("%s_b"), *Key.ToString());
		break;
	case EOnlineKeyValuePairDataType::Empty:
	case EOnlineKeyValuePairDataType::Blob:
	default:
		return false;
	}

	return true;
}

/**
 * Converts Steam key/value data back to its appropriate online key and its associated data
 * see SessionKeyToSteamKey() above
 *
 * @param SteamKey key in Steam string format
 * @param SteamValue data associated with the SteamKey
 * @param Key output key for use with online subsystem
 * @param Setting output data associated with Key
 * 
 * @return true if successful, false if data type unknown/unsupported
 */
inline bool SteamKeyToSessionSetting(const TCHAR* SteamKey, const ANSICHAR* SteamValue, FName& Key, FOnlineSessionSetting& Setting)
{
	bool bSuccess = false;

	TCHAR SteamKeyCopy[1024];

	FCString::Strncpy(SteamKeyCopy, SteamKey, UE_ARRAY_COUNT(SteamKeyCopy));

	TCHAR* DataType = FCString::Strrchr(SteamKeyCopy, '_');
	if (DataType)
	{
		bSuccess = true;

		// NULL Terminate the key
		*DataType = '\0';
		Key = FName(SteamKeyCopy);

		// Advance to the data type
		DataType += 1;

		switch(DataType[0])
		{
		case 'i':
			Setting.Data.SetValue((int32)0);
			Setting.Data.FromString(ANSI_TO_TCHAR(SteamValue));
			break;
		case 'l':
			Setting.Data.SetValue((uint64)0);
			Setting.Data.FromString(ANSI_TO_TCHAR(SteamValue));
			break;
		case 'd':
			Setting.Data.SetValue((double)0);
			Setting.Data.FromString(ANSI_TO_TCHAR(SteamValue));
			break;
		case 's':
			Setting.Data.SetValue(ANSI_TO_TCHAR(SteamValue));
			break;
		case 'f':
			Setting.Data.SetValue((float)0);
			Setting.Data.FromString(ANSI_TO_TCHAR(SteamValue));
			break;
		case 'b':
			Setting.Data.SetValue(false);
			Setting.Data.FromString(ANSI_TO_TCHAR(SteamValue));
			break;
		default:
			bSuccess = false;
		}
	}
	
	if (!bSuccess)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Unknown or unsupported data type from Steam key data %s %s"), SteamKey, ANSI_TO_TCHAR(SteamValue));
	}

	return bSuccess;
}
