// Copyright Epic Games, Inc. All Rights Reserved.

#include "SteamUtilities.h"
#include "OnlineSubsystemTypes.h"
#include <steam/steamtypes.h>

/**
 * Takes a Steam EResult value, and converts it into a string (with extra debug info)
 *
 * @param Result	The EResult value to convert to a string
* @return the converted string for the given result
 */
FString SteamResultString(EResult Result)
{
	FString ReturnVal;

	#define SteamResultCase(Value, Desc) \
		case Value: ReturnVal = FString::Printf(TEXT("'%i' %s (%s)"), (int32)Value, TEXT(#Value), Desc); break;

	switch (Result)
	{
	SteamResultCase(k_EResultOK,								TEXT("success"));
	SteamResultCase(k_EResultFail,								TEXT("failure"));
	SteamResultCase(k_EResultNoConnection,						TEXT("no connection"));
	SteamResultCase(k_EResultInvalidPassword,					TEXT("invalid password/ticket"));
	SteamResultCase(k_EResultLoggedInElsewhere,					TEXT("same user logged in elsewhere"));
	SteamResultCase(k_EResultInvalidProtocolVer,				TEXT("incorrect protocol version"));
	SteamResultCase(k_EResultInvalidParam,						TEXT("a parameter is incorrect"));
	SteamResultCase(k_EResultFileNotFound,						TEXT("file not found"));
	SteamResultCase(k_EResultBusy,								TEXT("called method busy, no action taken"));
	SteamResultCase(k_EResultInvalidState,						TEXT("called object in invalid state"));
	SteamResultCase(k_EResultInvalidName,						TEXT("invalid name"));
	SteamResultCase(k_EResultInvalidEmail,						TEXT("invalid email"));
	SteamResultCase(k_EResultDuplicateName,						TEXT("duplicate name"));
	SteamResultCase(k_EResultAccessDenied,						TEXT("access denied"));
	SteamResultCase(k_EResultTimeout,							TEXT("operation timed out"));
	SteamResultCase(k_EResultBanned,							TEXT("VAC banned"));
	SteamResultCase(k_EResultAccountNotFound,					TEXT("account not found"));
	SteamResultCase(k_EResultInvalidSteamID,					TEXT("steamid invalid"));
	SteamResultCase(k_EResultServiceUnavailable,				TEXT("requested service currently unavailable"));
	SteamResultCase(k_EResultNotLoggedOn,						TEXT("user is not logged on"));
	SteamResultCase(k_EResultPending,							TEXT("request is pending - may be in process, or waiting on third party"));
	SteamResultCase(k_EResultEncryptionFailure,					TEXT("encryption or decryption failed"));
	SteamResultCase(k_EResultInsufficientPrivilege,				TEXT("insufficient privilege"));
	SteamResultCase(k_EResultLimitExceeded,						TEXT("limit exceeded"));
	SteamResultCase(k_EResultRevoked,							TEXT("access revoked"));
	SteamResultCase(k_EResultExpired,							TEXT("license or guest pass expired"));
	SteamResultCase(k_EResultAlreadyRedeemed,					TEXT("guest pass already redeemed"));
	SteamResultCase(k_EResultDuplicateRequest,					TEXT("duplicate request, already occurred, ignoring"));
	SteamResultCase(k_EResultAlreadyOwned,						TEXT("already owned"));
	SteamResultCase(k_EResultIPNotFound,						TEXT("IP address not found"));
	SteamResultCase(k_EResultPersistFailed,						TEXT("failed to write change to data store"));
	SteamResultCase(k_EResultLockingFailed,						TEXT("failed to acquire access lock for operation"));
	SteamResultCase(k_EResultLogonSessionReplaced,				TEXT("???"));
	SteamResultCase(k_EResultConnectFailed,						TEXT("???"));
	SteamResultCase(k_EResultHandshakeFailed,					TEXT("???"));
	SteamResultCase(k_EResultIOFailure,							TEXT("input/output failure"));
	SteamResultCase(k_EResultRemoteDisconnect,					TEXT("???"));
	SteamResultCase(k_EResultShoppingCartNotFound,				TEXT("failed to find shopping cart requested"));
	SteamResultCase(k_EResultBlocked,							TEXT("blocked"));
	SteamResultCase(k_EResultIgnored,							TEXT("ignored"));
	SteamResultCase(k_EResultNoMatch,							TEXT("nothing matching request found"));
	SteamResultCase(k_EResultAccountDisabled,					TEXT("???"));
	SteamResultCase(k_EResultServiceReadOnly,					TEXT("service not accepting content changes right now"));
	SteamResultCase(k_EResultAccountNotFeatured,				TEXT("???"));
	SteamResultCase(k_EResultAdministratorOK,					TEXT("allowed to take this action, but only because requester is admin"));
	SteamResultCase(k_EResultContentVersion,					TEXT("version mismatch in transmitted content"));
	SteamResultCase(k_EResultTryAnotherCM,						TEXT("???"));
	SteamResultCase(k_EResultPasswordRequiredToKickSession,		TEXT("???"));
	SteamResultCase(k_EResultAlreadyLoggedInElsewhere,			TEXT("already logged in elsewhere, must wait"));
	SteamResultCase(k_EResultSuspended,							TEXT("operation suspended/paused"));
	SteamResultCase(k_EResultCancelled,							TEXT("operation cancelled"));
	SteamResultCase(k_EResultDataCorruption,					TEXT("operation cancelled due to corrupt data"));
	SteamResultCase(k_EResultDiskFull,							TEXT("operation cancelled due to lack of disk space"));
	SteamResultCase(k_EResultRemoteCallFailed,					TEXT("remote call or IPC call failed"));
	SteamResultCase(k_EResultPasswordUnset,						TEXT("password not verified, as it's unset serverside"));
	SteamResultCase(k_EResultExternalAccountUnlinked,			TEXT("external account not linked to a steam account"));
	SteamResultCase(k_EResultPSNTicketInvalid,					TEXT("PSN ticket invalid"));
	SteamResultCase(k_EResultExternalAccountAlreadyLinked,		TEXT("external account linked to other account"));
	SteamResultCase(k_EResultRemoteFileConflict,				TEXT("sync cannot resume, conflict between local and remote files"));
	SteamResultCase(k_EResultIllegalPassword,					TEXT("requested password not legal"));
	SteamResultCase(k_EResultSameAsPreviousValue,				TEXT("new value same as old"));
	SteamResultCase(k_EResultAccountLogonDenied,				TEXT("account login denied due to 2nd factor auth failure"));
	SteamResultCase(k_EResultCannotUseOldPassword,				TEXT("requested password not legal"));
	SteamResultCase(k_EResultInvalidLoginAuthCode,				TEXT("account login denied, invalid auth code"));
	SteamResultCase(k_EResultAccountLogonDeniedNoMail,			TEXT("account login denied due to 2nd factor auth failure"));
	SteamResultCase(k_EResultHardwareNotCapableOfIPT,			TEXT("???"));
	SteamResultCase(k_EResultIPTInitError,						TEXT("???"));
	SteamResultCase(k_EResultParentalControlRestricted,			TEXT("operation failed due to parental controls"));
	SteamResultCase(k_EResultFacebookQueryError,				TEXT("facebook query returned error"));
	SteamResultCase(k_EResultExpiredLoginAuthCode,				TEXT("account login denied, expired auth code"));
	SteamResultCase(k_EResultIPLoginRestrictionFailed,			TEXT("???"));
	SteamResultCase(k_EResultAccountLockedDown,					TEXT("???"));
	SteamResultCase(k_EResultAccountLogonDeniedVerifiedEmailRequired,	TEXT("???"));
	SteamResultCase(k_EResultNoMatchingURL,						TEXT("no matching URL"));

	default:
		ReturnVal = FString::Printf(TEXT("Unknown EResult result: %i (check Steam SDK)"), (int32)Result);
		break;
	}

	#undef SteamResultCase

	return ReturnVal;
}

/**
 * Takes a Steam EChatRoomEnterResponse value, and converts it into a string (with extra debug info)
 *
 * @param Response	The EChatRoomEnterResponse value to convert to a string
 * @return the converted string for the given response
 */
FString SteamChatRoomEnterResponseString(EChatRoomEnterResponse Response)
{
	FString ReturnVal;

	#define SteamResultCase(Value, Desc) \
		case Value: ReturnVal = FString::Printf(TEXT("'%i' %s (%s)"), (int32)Value, TEXT(#Value), Desc); break;

	switch (Response)
	{
	SteamResultCase(k_EChatRoomEnterResponseSuccess,			TEXT("Success"));
	SteamResultCase(k_EChatRoomEnterResponseDoesntExist,		TEXT("Chat doesn't exist (probably closed)"));
	SteamResultCase(k_EChatRoomEnterResponseNotAllowed,			TEXT("General Denied - You don't have the permissions needed to join the chat"));
	SteamResultCase(k_EChatRoomEnterResponseFull,				TEXT("Chat room has reached its maximum size"));
	SteamResultCase(k_EChatRoomEnterResponseError,				TEXT("Unexpected Error"));
	SteamResultCase(k_EChatRoomEnterResponseBanned,				TEXT("You are banned from this chat room and may not join"));
	SteamResultCase(k_EChatRoomEnterResponseLimited,			TEXT("Joining this chat is not allowed because you are a limited user (no value on account)"));
	SteamResultCase(k_EChatRoomEnterResponseClanDisabled,		TEXT("Attempt to join a clan chat when the clan is locked or disabled"));
	SteamResultCase(k_EChatRoomEnterResponseCommunityBan,		TEXT("Attempt to join a chat when the user has a community lock on their account"));
	SteamResultCase(k_EChatRoomEnterResponseMemberBlockedYou,	TEXT("Join failed - some member in the chat has blocked you from joining"));
	SteamResultCase(k_EChatRoomEnterResponseYouBlockedMember,	TEXT("Join failed - you have blocked some member already in the chat"));

	default:
		ReturnVal = FString::Printf(TEXT("Unknown EChatRoomEnterResponse result: %i (check Steam SDK)"), (int32)Response);
		break;
	};
	#undef SteamResultCase

	return ReturnVal;
}

/**
 * Takes a Steam EChatMemberStateChange value, and converts it into a string (with extra debug info)
 *
 * @param StateChange	The EChatMemberStateChange value to convert to a string
 * @return the converted string for the given state change
 */
FString SteamChatMemberStateChangeString(EChatMemberStateChange StateChange)
{
	FString ReturnVal;

	#define SteamResultCase(Value, Desc) \
		case Value: ReturnVal = FString::Printf(TEXT("'%i' %s (%s)"), (int32)Value, TEXT(#Value), Desc); break;

	switch (StateChange)
	{
	SteamResultCase(k_EChatMemberStateChangeEntered,			TEXT("Join chat"));
	SteamResultCase(k_EChatMemberStateChangeLeft,				TEXT("Left chat"));
	SteamResultCase(k_EChatMemberStateChangeDisconnected,		TEXT("Chat disconnect"));
	SteamResultCase(k_EChatMemberStateChangeKicked,				TEXT("Chat kicked"));
	SteamResultCase(k_EChatMemberStateChangeBanned,				TEXT("Chat kicked and banned"));
	
	default:
		ReturnVal = FString::Printf(TEXT("Unknown EChatMemberStateChange result: %i (check Steam SDK)"), (int32)StateChange);
		break;
	};
	#undef SteamResultCase

	return ReturnVal;
}

/**
 * Takes a Steam EMatchMakingServerResponse value, and converts it into a string (with extra debug info)
 *
 * @param Response	The EMatchMakingServerResponse value to convert to a string
 * @return the converted string for the given response
 */
FString SteamMatchMakingServerResponseString(EMatchMakingServerResponse Response)
{
	FString ReturnVal;

	#define SteamResultCase(Value, Desc) \
		case Value: ReturnVal = FString::Printf(TEXT("'%i' %s (%s)"), (int32)Value, TEXT(#Value), Desc); break;

	switch (Response)
	{
		SteamResultCase(eServerResponded,				TEXT("Server success"));
		SteamResultCase(eServerFailedToRespond,			TEXT("Server failed to respond"));
		SteamResultCase(eNoServersListedOnMasterServer,	TEXT("No servers match"));

	default:
		ReturnVal = FString::Printf(TEXT("Unknown EMatchMakingServerResponse result: %i (check Steam SDK)"), (int32)Response);
		break;
	};
	#undef SteamResultCase

	return ReturnVal;
}

/**
 * Converts a Steam EP2PSessionError value to a readable/descriptive string
 * @param InError	The EP2PSessionError value to convert to a string
 * @return the converted string for the given error
 */
FString SteamP2PConnectError(EP2PSessionError InError)
{
	FString ReturnVal;

#define SteamResultCase(Value, Desc) \
		case Value: ReturnVal = FString::Printf(TEXT("'%i' %s (%s)"), (int32)Value, TEXT(#Value), Desc); break;

	switch (InError)
	{
		// there are two other constants here but they are both deprecated/slated for removal
		SteamResultCase(k_EP2PSessionErrorNone,				TEXT("None"));
		SteamResultCase(k_EP2PSessionErrorNoRightsToApp,	TEXT("NoRightsToApp"));
		SteamResultCase(k_EP2PSessionErrorTimeout,			TEXT("Timeout"));

	default:
		ReturnVal = FString::Printf(TEXT("Unknown EP2PSessionError result: %i (check Steam SDK)"), (int32)InError);
		break;
	};
	#undef SteamResultCase

	return ReturnVal;
}

/**
 * Converts a Steam EVoiceResult value to a readable/descriptive string
 * @param Result	The EVoiceResult value to convert to a string
 * @return the converted string for the given error
 */
FString SteamVoiceResult(EVoiceResult Result)
{
	FString ReturnVal;

#define SteamResultCase(Value, Desc) \
		case Value: ReturnVal = FString::Printf(TEXT("'%i' %s (%s)"), (int32)Value, TEXT(#Value), Desc); break;

	switch (Result)
	{
		SteamResultCase(k_EVoiceResultOK,				TEXT("Voice OK"));
		SteamResultCase(k_EVoiceResultNotInitialized,	TEXT("Voice Not Initialized"));
		SteamResultCase(k_EVoiceResultNotRecording,		TEXT("Voice Not Recording"));
		SteamResultCase(k_EVoiceResultNoData,			TEXT("Voice No Data"));
		SteamResultCase(k_EVoiceResultBufferTooSmall,	TEXT("Voice Buffer Too Small"));
		SteamResultCase(k_EVoiceResultDataCorrupted,	TEXT("Voice Data Corrupted"));
		SteamResultCase(k_EVoiceResultRestricted,		TEXT("Voice Restricted"));
		SteamResultCase(k_EVoiceResultUnsupportedCodec,	TEXT("Voice Unsupported Codec"));

	default:
		ReturnVal = FString::Printf(TEXT("Unknown EVoiceResult result: %i (check Steam SDK)"), (int32)Result);
		break;
	};
#undef SteamResultCase

	return ReturnVal;
}

/**
 *	Takes a Steam EResult value, and converts it to an online connection state
 * @param Result	The EResult value to convert to a connection state
 * @return the connection state for the given result
 */
EOnlineServerConnectionStatus::Type SteamConnectionResult(const EResult Result)
{
	switch (Result)
	{
	case k_EResultAdministratorOK:
	case k_EResultOK: 
		return EOnlineServerConnectionStatus::Connected;

	case k_EResultNoConnection:
		return EOnlineServerConnectionStatus::NoNetworkConnection;

	case k_EResultInvalidPassword:
	case k_EResultNotLoggedOn:
	case k_EResultAccessDenied:
	case k_EResultBanned:
	case k_EResultAccountNotFound:
	case k_EResultInvalidSteamID:
	case k_EResultRevoked:
	case k_EResultExpired:
	case k_EResultAlreadyRedeemed:
	case k_EResultBlocked:
	case k_EResultIgnored:
	case k_EResultAccountDisabled:
	case k_EResultAccountNotFeatured:
	case k_EResultInsufficientPrivilege:
		return EOnlineServerConnectionStatus::InvalidUser;

	case k_EResultLogonSessionReplaced:
	case k_EResultRemoteDisconnect:
	case k_EResultLoggedInElsewhere:
		return EOnlineServerConnectionStatus::DuplicateLoginDetected;

	case k_EResultInvalidProtocolVer:
	case k_EResultContentVersion:
		return EOnlineServerConnectionStatus::UpdateRequired;

	case k_EResultBusy:
		return EOnlineServerConnectionStatus::ServersTooBusy;

	default:
		return EOnlineServerConnectionStatus::ServiceUnavailable;
	}
}
