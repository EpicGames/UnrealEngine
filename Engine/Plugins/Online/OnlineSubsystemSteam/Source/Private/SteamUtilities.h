// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

THIRD_PARTY_INCLUDES_START
// IWYU pragma: begin_exports
#include <steam/isteammatchmaking.h>
#include <steam/isteamnetworking.h>
#include <steam/matchmakingtypes.h>
#include <steam/steamclientpublic.h>
// IWYU pragma: end_exports
THIRD_PARTY_INCLUDES_END


class FString;
namespace EOnlineServerConnectionStatus { enum Type : uint8; }

/**
 * Takes a Steam EResult value, and converts it into a string (with extra debug info)
 *
 * @param Result	The EResult value to convert to a string
* @return the converted string for the given result
 */
FString SteamResultString(EResult Result);

/**
 * Takes a Steam EChatMemberStateChange value, and converts it into a string (with extra debug info)
 *
 * @param StateChange	The EChatMemberStateChange value to convert to a string
 * @return the converted string for the given state change
 */
FString SteamChatMemberStateChangeString(EChatMemberStateChange StateChange);

/**
 * Takes a Steam EChatRoomEnterResponse value, and converts it into a string (with extra debug info)
 *
 * @param Response	The EChatRoomEnterResponse value to convert to a string
 * @return the converted string for the given response
 */
FString SteamChatRoomEnterResponseString(EChatRoomEnterResponse Response);

/**
 * Takes a Steam EMatchMakingServerResponse value, and converts it into a string (with extra debug info)
 *
 * @param Response	The EMatchMakingServerResponse value to convert to a string
 * @return the converted string for the given response
 */
FString SteamMatchMakingServerResponseString(EMatchMakingServerResponse Response);

/**
 * Converts a Steam EP2PSessionError value to a readable/descriptive string
 * @param InError	The EP2PSessionError value to convert to a string
 * @return the converted string for the given error
 */
FString SteamP2PConnectError(EP2PSessionError InError);

/**
 * Converts a Steam EVoiceResult value to a readable/descriptive string
 * @param Result	The EVoiceResult value to convert to a string
 * @return the converted string for the given error
 */
FString SteamVoiceResult(EVoiceResult Result);

/**
 *	Takes a Steam EResult value, and converts it to an online connection state
 * @param Result	The EResult value to convert to a connection state
 * @return the connection state for the given result
 */
EOnlineServerConnectionStatus::Type SteamConnectionResult(const EResult Result);
