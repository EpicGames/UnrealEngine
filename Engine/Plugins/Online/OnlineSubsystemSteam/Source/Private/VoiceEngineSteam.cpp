// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_ENGINE
#include "VoiceEngineSteam.h"
#include "OnlineSubsystemSteamPrivate.h" // IWYU pragma: keep

FVoiceEngineSteam::FVoiceEngineSteam(IOnlineSubsystem* InSubsystem) :
	FVoiceEngineImpl(InSubsystem),
	SteamUserPtr(SteamUser()),
	SteamFriendsPtr(SteamFriends())
{
}

FVoiceEngineSteam::~FVoiceEngineSteam()
{
	if (IsRecording())
	{
		SteamFriendsPtr->SetInGameVoiceSpeaking(SteamUserPtr->GetSteamID(), false);
	}
}

void FVoiceEngineSteam::StartRecording() const
{
	UE_LOG_ONLINE_VOICEENGINE(VeryVerbose, TEXT("VOIP StartRecording"));
	if (GetVoiceCapture().IsValid())
	{
		if (!GetVoiceCapture()->Start())
		{
			UE_LOG_ONLINE_VOICEENGINE(Warning, TEXT("Failed to start voice recording"));
		}
		else if (SteamFriendsPtr) 
		{
			SteamFriendsPtr->SetInGameVoiceSpeaking(SteamUserPtr->GetSteamID(), true);
		}
	}
}

void FVoiceEngineSteam::StoppedRecording() const
{
	UE_LOG_ONLINE_VOICEENGINE(VeryVerbose, TEXT("VOIP StoppedRecording"));
	if (SteamFriendsPtr)
	{
		SteamFriendsPtr->SetInGameVoiceSpeaking(SteamUserPtr->GetSteamID(), false);
	}
}
#endif //WITH_ENGINE