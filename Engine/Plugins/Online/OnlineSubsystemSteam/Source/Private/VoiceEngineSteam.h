// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_ENGINE
#include "VoiceEngineImpl.h"

class IOnlineSubsystem;
class FUniqueNetIdSteam;

#define INVALID_INDEX -1

/**
* Generic implementation of voice engine, using Voice module for capture/codec
*/
class FVoiceEngineSteam : public FVoiceEngineImpl
{
	/** Steam User interface */
	class ISteamUser* SteamUserPtr;
	/** Steam Friends interface */
	class ISteamFriends* SteamFriendsPtr;

	/** Start capturing voice data */
	virtual void StartRecording() const override;

	/** Called when "last half second" is over */
	virtual void StoppedRecording() const override;

PACKAGE_SCOPE:

	/** Constructor */
	FVoiceEngineSteam() :
		FVoiceEngineImpl(),
		SteamUserPtr(nullptr),
		SteamFriendsPtr(nullptr)
	{};

public:

	FVoiceEngineSteam(IOnlineSubsystem* InSubsystem);
	virtual ~FVoiceEngineSteam();
};
#endif //WITH_ENGINE