// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_ENGINE
#include "VoiceInterfaceImpl.h"
#include "VoiceEngineSteam.h"

/**
 * The Steam implementation of the voice interface 
 */

class FOnlineVoiceSteam : public FOnlineVoiceImpl
{
PACKAGE_SCOPE:
	FOnlineVoiceSteam() : FOnlineVoiceImpl()
	{};

public:

	/** Constructor */
	FOnlineVoiceSteam(class IOnlineSubsystem* InOnlineSubsystem) :
		FOnlineVoiceImpl(InOnlineSubsystem)
	{
		check(InOnlineSubsystem);
	};

	virtual IVoiceEnginePtr CreateVoiceEngine() override
	{
		return MakeShareable(new FVoiceEngineSteam(OnlineSubsystem));
	}

	/** Virtual destructor to force proper child cleanup */
	virtual ~FOnlineVoiceSteam() override {};
};

typedef TSharedPtr<FOnlineVoiceSteam, ESPMode::ThreadSafe> FOnlineVoiceSteamPtr;
#endif //WITH_ENGINE
