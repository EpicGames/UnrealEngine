// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_ENGINE

#include "VoicePacketSteam.h"
#include "OnlineSubsystemSteamTypes.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FVoicePacketSteam::FVoicePacketSteam(const FVoicePacketSteam& Other) :
	FVoicePacket(Other)
{
	Sender = Other.Sender;
	Length = Other.Length;

	// Copy the contents of the voice packet
	Buffer.Empty(Other.Length);
	Buffer.AddUninitialized(Other.Length);
	FMemory::Memcpy(Buffer.GetData(), Other.Buffer.GetData(), Other.Length);
}

uint16 FVoicePacketSteam::GetTotalPacketSize()
{
	return Sender->GetSize() + sizeof(Length) + Length;
}

uint16 FVoicePacketSteam::GetBufferSize()
{
	return Length;
}

FUniqueNetIdPtr FVoicePacketSteam::GetSender()
{
	return Sender;
}

void FVoicePacketSteam::Serialize(class FArchive& Ar)
{
	// Make sure not to overflow the buffer by reading an invalid amount
	if (Ar.IsLoading())
	{
		uint64 SenderUID;
		Ar << SenderUID;
		Sender = FUniqueNetIdSteam::Create(SenderUID);
		Ar << Length;
		// Verify the packet is a valid size
		if (Length <= MAX_VOICE_DATA_SIZE)
		{
			Buffer.Empty(Length);
			Buffer.AddUninitialized(Length);
			Ar.Serialize(Buffer.GetData(), Length);
		}
		else
		{
			Length = 0;
		}
	}
	else
	{
		Ar << *(uint64*)Sender->GetBytes();
		Ar << Length;

		// Always safe to save the data as the voice code prevents overwrites
		Ar.Serialize(Buffer.GetData(), Length);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif //WITH_ENGINE