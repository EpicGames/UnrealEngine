// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_ENGINE
#pragma once

#include "Net/VoiceDataCommon.h"
#include "OnlineSubsystemSteamPackage.h"

/** Defines the data involved in a Steam voice packet */
class UE_DEPRECATED(4.25, "Steam Voice is now handled by the default voice implementation, FOnlineVoiceImpl. This code is no longer used.") FVoicePacketSteam : public FVoicePacket
{

PACKAGE_SCOPE:

	/** The unique net id of the talker sending the data */
	FUniqueNetIdPtr Sender;
	/** The data that is to be sent/processed */
	TArray<uint8> Buffer;
	/** The current amount of space used in the buffer for this packet */
	uint16 Length;

public:
	/** Zeros members and validates the assumptions */
	FVoicePacketSteam() :
		Sender(NULL),
		Length(0)
	{
		Buffer.Empty(MAX_VOICE_DATA_SIZE);
	}

	/** Should only be used by TSharedPtr and FVoiceData */
	virtual ~FVoicePacketSteam()
	{
	}

	/**
	 * Copies another packet
	 *
	 * @param Other packet to copy
	 */
	FVoicePacketSteam(const FVoicePacketSteam& Other);

	//~ Begin FVoicePacket interface
	virtual uint16 GetTotalPacketSize() override;
	virtual uint16 GetBufferSize() override;
	virtual FUniqueNetIdPtr GetSender() override;
	virtual bool IsReliable() override { return false; }
	virtual void Serialize(class FArchive& Ar) override;
	//~ End FVoicePacket interface
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
struct FVoiceDataSteam_DEPRECATED
{
	/** Data used by the local talkers before sent */
	FVoicePacketSteam LocalPackets[MAX_SPLITSCREEN_TALKERS];
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS


/** Holds the current voice packet data state */
struct UE_DEPRECATED(4.25, "Voice functionality on Steam no longer requires platform specific code.") FVoiceDataSteam : public FVoiceDataSteam_DEPRECATED
{
	/** Holds the set of received packets that need to be processed */
	FVoicePacketList RemotePackets;

	FVoiceDataSteam() {}
	~FVoiceDataSteam() {}
};
#endif //WITH_ENGINE