// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Status flags of SteamAuth response codes, mapped 1:1 with SteamAPI */
enum class ESteamAuthResponseCode : uint8
{
	OK = 0,
	NotConnectedToSteam = 1,
	NoOwnership = 2,
	VACBanned = 3,
	LoggedInElsewhere = 4,
	VACCheckTimeout = 5,
	TicketCancelled = 6,
	TicketAlreadyUsed = 7,
	TicketInvalid = 8,
	PublisherBanned = 9,
	// ... space given for future steam codes ...
	FailedToCreateUser = 20,
	Count
};

/** When authentication has failed and we are about to take action on the user, this delegate is fired. 
 *	For the auth interface, overriding the delegate exposed in the class allows a game to override the default
 *	behavior, which is to kick anyone who fails authentication.
 *	
 *	If you would like to receive analytics as to the success/failure for users we can identify
 *	(have their unique net id), use the result delegate instead.
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnSteamAuthFailure, const class FUniqueNetId& /*FailedUserId*/);
typedef FOnSteamAuthFailure::FDelegate FOnSteamAuthFailureDelegate;

/** This delegate dictates the success or failure of an authentication result. 
 *  This means we got a result, but we won't be taking action yet.
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSteamAuthResult, const class FUniqueNetId& /*UserId*/, bool /*bWasSuccess*/);
typedef FOnSteamAuthResult::FDelegate FOnSteamAuthResultDelegate;

/** This delegate dictates the success or failure of an authentication result with an additional flag that explains the result.
 *  This is fired at the same time as the above delegate but includes additional data.
 *  Should a project be listening to result delegate calls, it is recommended to listen to one of these two (but not both).
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnSteamAuthResultWithCode, const class FUniqueNetId& /*UserId*/, bool /*bWasSuccess*/, ESteamAuthResponseCode /*ResponseCode*/);
typedef FOnSteamAuthResultWithCode::FDelegate FOnSteamAuthResultWithCodeDelegate;

/** A class for interfacing with SteamAuth outside of the SteamOSS. */
class FOnlineAuthUtilsSteam
{
public:
	
	~FOnlineAuthUtilsSteam()
	{
		OverrideFailureDelegate.Unbind();
		OnAuthenticationResultDelegate.Unbind();
		OnAuthenticationResultWithCodeDelegate.Unbind();
	}	
	
	/**
	 * Checks to see if Steam Authentication is enabled due to being defined
	 * in any PacketHandlerComponent list.
	 *
	 * @return the true if SteamAuth is enabled.
	 *
	 */
	ONLINESUBSYSTEMSTEAM_API bool IsSteamAuthEnabled() const;
	
	/** Attach to this delegate to control the behavior of the Steam authentication failure. 
	 *  This overrides the default behavior (kick). 
	 */
	FOnSteamAuthFailureDelegate OverrideFailureDelegate;
	FOnSteamAuthResultDelegate OnAuthenticationResultDelegate;
	FOnSteamAuthResultWithCodeDelegate OnAuthenticationResultWithCodeDelegate;
};
