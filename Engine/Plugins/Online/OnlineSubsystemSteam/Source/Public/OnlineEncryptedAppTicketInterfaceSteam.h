// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "OnlineSubsystemSteam.h"

#define UE_API ONLINESUBSYSTEMSTEAM_API

class FOnlineSubsystemSteam;
enum class ESteamEncryptedAppTicketState;

/**	Steam Encrypted Application Ticket Interface.
 *
 *	You can created encrypted Steam application tickets with optional data by calling the RequestEncryptedAppTicket()
 *	function, which after the result delegate firing must be waited before the encrypted application ticket
 *	is available for use.
 *
 *	Once the encrypted steam application ticket is available for use, it can be retrieved via GetEncryptedAppTicket().
 *
 *	Do note that Steam limits the encrypted application ticket frequency to one per 60 seconds.
 *	Trying to encrypt data more often will result in an k_EResultLimitExceeded error.
 */

/**	This delegate dictates the success or failure of data encrypting result.
 *	On success, this means that the encrypted data is now available and can be retrieved via call to GetEncryptedAppTicket.
 *
 * @param bEncryptedDataAvailable - True if function call was a success and the data is available, else false.
 * @param ResultCode - Steam API EResult code describing the result of the API call.
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnEncryptedAppTicketResponse, bool /*bEncryptedDataAvailable*/, int32 /*ResultCode*/);

class FOnlineEncryptedAppTicketSteam :
	public FSelfRegisteringExec
{
PACKAGE_SCOPE:
	FOnlineEncryptedAppTicketSteam(FOnlineSubsystemSteam* InSubsystem);
	UE_API virtual ~FOnlineEncryptedAppTicketSteam();

	/**
	 * Callback function informed of a result of the latest successful call to RequestEncryptedAppTicket.
	 *
	 * @param bEncryptedDataAvailable - True if function call was a success and the data is available, else false.
	 * @param ResultCode - Steam API EResult code describing the result of the API call.
	 */
	void OnAPICallComplete(bool bEncryptedDataAvailable, int32 ResultCode);

public:
	/**
	 * Requests data encrypting with Steam Encrypted Application ticket API.
	 *
	 * The encryption key used is the one configured on the App Admin page for your app on Steam portal.
	 * There can only be one data encrypting request pending, and this call is subject to a 60 second rate limit.
	 *
	 * @param DataToEncrypt - Data to encrypt.
	 * @param SizeOfDataToEncrypt - Length of data to encrypt.
	 * @return True on success, false on failure.
	 */
	UE_API bool RequestEncryptedAppTicket(void* DataToEncrypt, int SizeOfDataToEncrypt);

	/**
	 * Retrieve encrypted data once it's available.
	 *
	 * @param OutEncryptedData - Upon successful call, contains the encrypted application ticket data.
	 * @return True if the function call was successful, otherwise false.
	 * @warning Only call this after OnDataEncryptResultDelegate has reported the data to be available.
	 */
	UE_API bool GetEncryptedAppTicket(TArray<uint8>& OutEncryptedData);

	/* Attach to this delegate to get notified about the encrypted application ticket results. */
	FOnEncryptedAppTicketResponse OnEncryptedAppTicketResultDelegate;

protected:
	// FSelfRegisteringExec
	UE_API virtual bool Exec_Dev(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;

private:

	/** Reference to the main Steam subsystem */
	FOnlineSubsystemSteam* SteamSubsystem;

	/** Describes the state of the object. */
	ESteamEncryptedAppTicketState TicketState;
};

#undef UE_API
