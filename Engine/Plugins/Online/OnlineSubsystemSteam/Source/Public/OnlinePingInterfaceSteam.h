// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * An abstract ping interface for Steam ideally usable with the new 
 * SteamSocket internet protocol
 */
class FOnlinePingInterfaceSteam
{
public:
	FOnlinePingInterfaceSteam(class FOnlineSubsystemSteam* InSubsystem) :
		Subsystem(InSubsystem)
	{
	}

	virtual ~FOnlinePingInterfaceSteam()
	{
	}
	
	/**
	 * Returns if this application is set up to use the Steam P2P Relay Network
	 * for communication.
	 *
	 * Uses OnlineSubsystemSteam.bAllowP2PPacketRelay
	 *
	 * @return true if relays are enabled for P2P connections.
	 */
	virtual bool IsUsingP2PRelays() const = 0;

	/**
	 * Returns the P2P relay ping information for the current machine. This information can be
	 * serialized over the network and used to calculate the ping data between a client and a host.
	 *
	 * @return relay information blob stored as a string for relaying over the network. 
	 *         If an error occurred, the return is an empty string.
	 */
	virtual FString GetHostPingData() const = 0;

	/**
	 * Calculates the ping of this client using the given host's ping data obtained from GetHostPingData.
	 * 
	 * @param HostPingStr The relay information blob we got from the host. This information
	 *                    should be directly serialized over the network and not tampered with.
	 *
	 * @return The ping value to the given host if it can be calculated, otherwise -1 on error.
	 */
	virtual int32 GetPingFromHostData(const FString& HostPingStr) const = 0;

	/**
	* An informative member that allows us to check if we are recalculating our ping
	* information over the Valve network. Data is additively modified during recalculation
	* such that we do not need to block on this function returning false before using
	* ping data.
	*
	* @return true if we're recalculating our ping within the Valve relay network.
	*/
	virtual bool IsRecalculatingPing() const = 0;
	
protected:
	FOnlineSubsystemSteam* Subsystem;
};
