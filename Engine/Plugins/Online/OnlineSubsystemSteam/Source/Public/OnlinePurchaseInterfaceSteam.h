// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemSteamPrivate.h"
#include "Interfaces/OnlinePurchaseInterface.h"
#include "OnlineSubsystemSteamTypes.h"
#include "Interfaces/OnlineStoreInterfaceV2.h"

#define USE_STEAM_DEBUG_PURCHASING_LINK 1

class FOnlineSubsystemSteam;

/*
*  A good amount of Steam purchasing and entitlements cannot be done entirely in-game, it requires the use of a backend server to use the web api for some flows.
*  This interface serves as a method of preserving the universal platform interface for the Online Subsystems while allowing you to add custom backend code required for full functionality.
*  Epic does not have an officially licensed solution for the implementation required in this server.
* 
*  Note: Microtransactions made using the server link will not show up via QueryReceipts
*/
/*struct FSteamMtxnDef
{

public:
	FOfferNamespace Namespace;
	FUniqueOfferId OfferId;
	FString Description;
	FString CostString;
	FString Category;
};*/

struct FSteamPurchaseDef
{

public:
	FOnlineStoreOfferRef Mtxn;
	int32_t Quantity;
};

/**
 * Delegate called when the server wants to update our list of offers.
 *
 * @param bWasSuccessful whether the async call completed properly or not
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnOffersUpdated, TArray<FString> /*Categories*/, TArray<FOnlineStoreOfferRef> /*Mtxns*/);
typedef FOnOffersUpdated::FDelegate FOnOffersUpdatedDelegate;

class ISteamPurchasingServerLink
{

public:
	virtual ~ISteamPurchasingServerLink() = default;

	/*
	*	Required virtual function, and the main use case of the server link.
	*   Use this to call into your server to use the Steam Web API and complete the microtransaction purchase request.
	*   Your backend server should be using the InitTxn API
	*/
	virtual void InitiateTransaction(const FUniqueNetId& UserId, TArray<FSteamPurchaseDef> Mtxns, const FOnPurchaseCheckoutComplete& Delegate) = 0;

	/*
	*	To use offer enumeration in the purchase interface, you must call these delegates after connecting the server link to refresh the purchase and store interface's offers.
	*/
	FOnOffersUpdatedDelegate OffersUpdatedDelegate;

	// Optional Functions
	
	/*
	*  Overrride this function if your game uses the FinalizePurchase functionality to consume a microtransaction and mark it as properly used.
	*  Your backend server should be calling the FinalizeTxn API, if it did not do so as part of the InitiateTransaction flow
	*/
	virtual void FinalizePurchase(const FUniqueNetId& UserId, const FString& ReceiptId) {
		UE_LOG_ONLINE_PURCHASE(Error, TEXT("FinalizePurchase called but server link does not override!"))
	}

	/*
	*  Overrride this function if your game uses the FinalizeReceiptValidationInfo. 
	*  The Steam API does not directly provide anything useful for this call (other platforms would provide something like a JWT token). It is generally recommended to simply make that connection on the backend using the web API
	*/
	virtual void FinalizeReceiptValidationInfo(const FUniqueNetId& UserId, FString& InReceiptValidationInfo, const FOnFinalizeReceiptValidationInfoComplete& Delegate) {
		UE_LOG_ONLINE_PURCHASE(Error, TEXT("FinalizeReceiptValidationInfo called but server link does not override!"))
	}
};

/* 
* The Steam purchase interface is designed to handle microtransactions in-game, not DLC entitlements (use the external UI to show the Steam store for those)
* Steam microtransactions are not defined by the server, but rather by the client. UE lets us define these microtransactions in two ways
* 
*  1) Config Driven
*		Config driven microtransactions can be datamined, and can possibily be tampered if your backend server does not have the proper protections in place to verify the transaction. Therefore it is only recommended for development/debug flows.
*		To add a new static microtransaction to Steam, add the following in your game's Engine file
*			
*				[OnlineSubsystemSteam]
*					+StaticMicrotransactions=(Id=32bitinteger, Amount=costofitem, Description=myMicroTransaction) 
* 
*  2) Server-driven
*		- Make an implementation of ISteamPurchasingServerLink and implement all functions
*/

class FOnlinePurchaseSteam
	: public IOnlinePurchase
	, public TSharedFromThis<FOnlinePurchaseSteam, ESPMode::ThreadSafe>
{
public:
	FOnlinePurchaseSteam(FOnlineSubsystemSteam* InSteamSubsystem);
	virtual ~FOnlinePurchaseSteam();

public:
	//~ Begin IOnlinePurchase Interface
	virtual bool IsAllowedToPurchase(const FUniqueNetId& UserId) override;
	virtual bool IsAllowedToPurchase(const FUniqueNetId& UserId, FOnlineError& Error) override;
	virtual void Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseCheckoutComplete& Delegate) override;
	virtual void Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseReceiptlessCheckoutComplete& Delegate) override;
	virtual void FinalizePurchase(const FUniqueNetId& UserId, const FString& ReceiptId) override;
	virtual void RedeemCode(const FUniqueNetId& UserId, const FRedeemCodeRequest& RedeemCodeRequest, const FOnPurchaseRedeemCodeComplete& Delegate) override;
	virtual void QueryReceipts(const FUniqueNetId& UserId, bool bRestoreReceipts, const FOnQueryReceiptsComplete& Delegate) override;
	virtual void GetReceipts(const FUniqueNetId& UserId, TArray<FPurchaseReceipt>& OutReceipts) const override;
	virtual void FinalizeReceiptValidationInfo(const FUniqueNetId& UserId, FString& InReceiptValidationInfo, const FOnFinalizeReceiptValidationInfoComplete& Delegate) override;
	//~ End IOnlinePurchase Interface

	void RegisterServerLink(TSharedRef<ISteamPurchasingServerLink> InServerLink);

	void GetStoreCategories(TArray<FOnlineStoreCategory>& OutCategories) const;
	void GetStoreOffers(TArray<FOnlineStoreOfferRef>& OutOffers) const;
	TSharedPtr<FOnlineStoreOffer> GetStoreOffer(const FUniqueOfferId& OfferId) const;

private:
	FOnlineSubsystemSteam* SteamSubsystem;
	TSharedPtr<ISteamPurchasingServerLink> ServerLink;

	TArray<FOnlineStoreOfferRef> CachedMtxns;
	TArray<FString> CachedCategories;
	TArray<FPurchaseReceipt> CachedDLC;

	void OnServerLinkOffersUpdated(TArray<FString> Categories, TArray<FOnlineStoreOfferRef> Mtxns);

};


#if USE_STEAM_DEBUG_PURCHASING_LINK

class FSteamPurchasingServerLinkDebug
	: public ISteamPurchasingServerLink
{
public:
	FSteamPurchasingServerLinkDebug();
	virtual ~FSteamPurchasingServerLinkDebug() = default;

	virtual void InitiateTransaction(const FUniqueNetId& UserId, TArray<FSteamPurchaseDef> Mtxns, const FOnPurchaseCheckoutComplete& Delegate) override;

	TArray<FOnlineStoreOfferRef> ConfigMtxns;
	TArray<FString> ConfigCategories;
};

#endif //USE_STEAM_DEBUG_PURCHASING_LINK