// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OnlineDelegateMacros.h"
#include "OnlineSubsystemImpl.h"
#include "OnlineSubsystemSteamPackage.h"

#define UE_API ONLINESUBSYSTEMSTEAM_API

class FOnlineAchievementsSteam;
class FOnlineExternalUISteam;
class FOnlineFriendsSteam;
class FOnlineIdentitySteam;
class FOnlineLeaderboardsSteam;
class FOnlineSessionSteam;
class FOnlineSharedCloudSteam;
class FOnlineUserCloudSteam;
class FOnlineVoiceSteam;
class FOnlinePresenceSteam;
class FOnlineAuthSteam;
class FOnlineAuthUtilsSteam;
class FOnlinePingInterfaceSteam;
class FOnlineEncryptedAppTicketSteam;
class FOnlinePurchaseSteam;
class FOnlineStoreSteam;

/** Forward declarations of all interface classes */
typedef TSharedPtr<class FOnlineSessionSteam, ESPMode::ThreadSafe> FOnlineSessionSteamPtr;
typedef TSharedPtr<class FOnlineIdentitySteam, ESPMode::ThreadSafe> FOnlineIdentitySteamPtr;
typedef TSharedPtr<class FOnlineFriendsSteam, ESPMode::ThreadSafe> FOnlineFriendsSteamPtr;
typedef TSharedPtr<class FOnlineSharedCloudSteam, ESPMode::ThreadSafe> FOnlineSharedCloudSteamPtr;
typedef TSharedPtr<class FOnlineUserCloudSteam, ESPMode::ThreadSafe> FOnlineUserCloudSteamPtr;
typedef TSharedPtr<class FOnlineLeaderboardsSteam, ESPMode::ThreadSafe> FOnlineLeaderboardsSteamPtr;
typedef TSharedPtr<class FOnlineVoiceSteam, ESPMode::ThreadSafe> FOnlineVoiceSteamPtr;
typedef TSharedPtr<class FOnlineExternalUISteam, ESPMode::ThreadSafe> FOnlineExternalUISteamPtr;
typedef TSharedPtr<class FOnlineAchievementsSteam, ESPMode::ThreadSafe> FOnlineAchievementsSteamPtr;
typedef TSharedPtr<class FOnlinePresenceSteam, ESPMode::ThreadSafe> FOnlinePresenceSteamPtr;
typedef TSharedPtr<class FOnlineAuthSteam, ESPMode::ThreadSafe> FOnlineAuthSteamPtr;
typedef TSharedPtr<class FOnlineAuthUtilsSteam, ESPMode::ThreadSafe> FOnlineAuthSteamUtilsPtr;
typedef TSharedPtr<class FOnlinePingInterfaceSteam, ESPMode::ThreadSafe> FOnlinePingSteamPtr;
typedef TSharedPtr<class FOnlineEncryptedAppTicketSteam, ESPMode::ThreadSafe> FOnlineEncryptedAppTicketSteamPtr;
typedef TSharedPtr<class FOnlinePurchaseSteam, ESPMode::ThreadSafe> FOnlinePurchaseSteamPtr;
typedef TSharedPtr<class FOnlineStoreSteam, ESPMode::ThreadSafe> FOnlineStoreSteamPtr;

/**
* Delegate fired when a Steam Game Server has completed its login tasks with the Steam backend.
*
* @param bWasSuccessful if the login completed successfully
*/
DECLARE_MULTICAST_DELEGATE_OneParam(FOnSteamServerLoginCompleted, bool /* bWasSuccessful */);
typedef FOnSteamServerLoginCompleted::FDelegate FOnSteamServerLoginCompletedDelegate;


/**
 *	OnlineSubsystemSteam - Implementation of the online subsystem for STEAM services
 */
class FOnlineSubsystemSteam : 
	public FOnlineSubsystemImpl
{
protected:

	/** Has the STEAM client APIs been initialized */
	bool bSteamworksClientInitialized;

	/** Whether or not the Steam game server API is initialized */
	bool bSteamworksGameServerInitialized;

	/** If we are using the SteamNetworking protocol or not. */
	bool bUsingSteamNetworking;

	/** Steam App ID for the running game */
	uint32 SteamAppID;

	/** Game port - the port that clients will connect to for gameplay */
	int32 GameServerGamePort;

	/** Query port - the port that will manage server browser related duties and info */
	int32 GameServerQueryPort;

	/** Array of the files in the cloud for a given user */
	TArray<struct FSteamUserCloudData*> UserCloudData;

	/** Interface to the session services */
	FOnlineSessionSteamPtr SessionInterface;

	/** Interface to the profile services */
	FOnlineIdentitySteamPtr IdentityInterface;

	/** Interface to the friend services */
	FOnlineFriendsSteamPtr FriendInterface;

	/** Interface to the shared cloud services */
	FOnlineSharedCloudSteamPtr SharedCloudInterface;
	
	/** Interface to the user cloud services */
	FOnlineUserCloudSteamPtr UserCloudInterface;

	/** Interface to the leaderboard services */
	FOnlineLeaderboardsSteamPtr LeaderboardsInterface;

	/** Interface to the voice engine */
	mutable IOnlineVoicePtr VoiceInterface;

	/** Interface for voice communication */
	mutable bool bVoiceInterfaceInitialized;

	/** Interface to the external UI services */
	FOnlineExternalUISteamPtr ExternalUIInterface;

	/** Interface for achievements */
	FOnlineAchievementsSteamPtr AchievementsInterface;

	/** Interface for presence */
	FOnlinePresenceSteamPtr PresenceInterface;

	/** Interface for Steam Session Auth */
	FOnlineAuthSteamPtr AuthInterface;
	FOnlineAuthSteamUtilsPtr AuthInterfaceUtils;

	/** Interface for dynamically calculating SteamNetworking ping based off protocol */
	FOnlinePingSteamPtr PingInterface;

	/** Interface for Steam encrypted application tickets. */
	FOnlineEncryptedAppTicketSteamPtr EncryptedAppTicketInterface;

	/** Interface for the Purchase interface */
	FOnlinePurchaseSteamPtr PurchaseInterface;

	/** Interface for the Store interface */
	FOnlineStoreSteamPtr StoreInterface;

	/** Online async task runnable */
	class FOnlineAsyncTaskManagerSteam* OnlineAsyncTaskThreadRunnable;

	/** Online async task thread */
	class FRunnableThread* OnlineAsyncTaskThread;

	/** Steam Client API Handle */
	TSharedPtr<class FSteamClientInstanceHandler> SteamAPIClientHandle;

	/** Steam Server API Handle */
	TSharedPtr<class FSteamServerInstanceHandler> SteamAPIServerHandle;

PACKAGE_SCOPE:

	/** Only the factory makes instances */
	FOnlineSubsystemSteam() = delete;
	FOnlineSubsystemSteam(FName InInstanceName) :
		FOnlineSubsystemImpl(STEAM_SUBSYSTEM, InInstanceName),
		bSteamworksClientInitialized(false),
		bSteamworksGameServerInitialized(false),
		bUsingSteamNetworking(false),
		SteamAppID(0),
		GameServerGamePort(0),
		GameServerQueryPort(0),
		SessionInterface(nullptr),
		IdentityInterface(nullptr),
		FriendInterface(nullptr),
		SharedCloudInterface(nullptr),
		UserCloudInterface(nullptr),
		LeaderboardsInterface(nullptr),
		VoiceInterface(nullptr),
		bVoiceInterfaceInitialized(false),
		ExternalUIInterface(nullptr),
		PresenceInterface(nullptr),
		AuthInterface(nullptr),
		AuthInterfaceUtils(nullptr),
		PingInterface(nullptr),
		EncryptedAppTicketInterface(nullptr),
		PurchaseInterface(nullptr),
		StoreInterface(nullptr),
		OnlineAsyncTaskThreadRunnable(nullptr),
		OnlineAsyncTaskThread(nullptr),
		SteamAPIClientHandle(nullptr),
		SteamAPIServerHandle(nullptr)
	{}

	/** Critical sections for thread safe operation of the cloud files */
	FCriticalSection UserCloudDataLock;

	/**
	 *	Initialize the client side APIs for Steam 
	 * @return true if the API was initialized successfully, false otherwise
	 */
	UE_API bool InitSteamworksClient(bool bRelaunchInSteam, int32 SteamAppId);

	/**
	 *	Initialize the server side APIs for Steam 
	 * @return true if the API was initialized successfully, false otherwise
	 */
	UE_API bool InitSteamworksServer();

	/**
	 *	Shutdown the Steam APIs
	 */
	UE_API void ShutdownSteamworks();
	
	/**
	 *	Add an async task onto the task queue for processing
	 * @param AsyncTask - new heap allocated task to process on the async task thread
	 */
	UE_API void QueueAsyncTask(class FOnlineAsyncTask* AsyncTask);

	/**
	 *	Add an async task onto the outgoing task queue for processing
	 * @param AsyncItem - new heap allocated task to process on the async task thread
	 */
	UE_API void QueueAsyncOutgoingItem(class FOnlineAsyncItem* AsyncItem);

	/** 
	 * **INTERNAL**
	 * Get the metadata related to a given user
	 * This information is only available after calling EnumerateUserFiles
	 *
	 * @param UserId the UserId to search for
	 * @return the struct with the metadata about the requested user, will always return a valid struct, creating one if necessary
	 *
	 */
	UE_API struct FSteamUserCloudData* GetUserCloudEntry(const FUniqueNetId& UserId);

	/** 
	 * **INTERNAL**
	 * Clear the metadata related to a given user's file on Steam
	 * This information is only available after calling EnumerateUserFiles
	 * It doesn't actually delete any of the actual data on disk
	 *
	 * @param UserId the UserId for the file to search for
	 * @param Filename the file to get metadata about
	 * @return the true if the delete was successful, false otherwise
	 *
	 */
	UE_API bool ClearUserCloudMetadata(const FUniqueNetId& UserId, const FString& Filename);

	/**
	 *	Clear out all the data related to user cloud storage
	 */
	UE_API void ClearUserCloudFiles();

	/** 
	 * **INTERNAL** 
	 * Get the interface for accessing leaderboards/stats
	 *
	 * @return pointer for the appropriate class
	 */
	UE_API FOnlineLeaderboardsSteam * GetInternalLeaderboardsInterface();

public:

	virtual ~FOnlineSubsystemSteam()
	{
	}

	UE_API virtual FOnlineEncryptedAppTicketSteamPtr GetEncryptedAppTicketInterface() const;

	UE_API virtual FOnlineAuthSteamPtr GetAuthInterface() const;
	UE_API virtual FOnlineAuthSteamUtilsPtr GetAuthInterfaceUtils() const;
	
	UE_API virtual FOnlinePingSteamPtr GetPingInterface() const;
	UE_API virtual void SetPingInterface(FOnlinePingSteamPtr InPingInterface);

	// IOnlineSubsystem

	UE_API virtual IOnlineSessionPtr GetSessionInterface() const override;
	UE_API virtual IOnlineFriendsPtr GetFriendsInterface() const override;
	UE_API virtual IOnlinePartyPtr GetPartyInterface() const override;
	UE_API virtual IOnlineGroupsPtr GetGroupsInterface() const override;
	UE_API virtual IOnlineSharedCloudPtr GetSharedCloudInterface() const override;
	UE_API virtual IOnlineUserCloudPtr GetUserCloudInterface() const override;
	UE_API virtual IOnlineLeaderboardsPtr GetLeaderboardsInterface() const override;
	UE_API virtual IOnlineVoicePtr GetVoiceInterface() const override;
	UE_API virtual IOnlineExternalUIPtr GetExternalUIInterface() const override;
	UE_API virtual IOnlineTimePtr GetTimeInterface() const override;
	UE_API virtual IOnlineIdentityPtr GetIdentityInterface() const override;
	UE_API virtual IOnlineTitleFilePtr GetTitleFileInterface() const override;
	UE_API virtual IOnlineEntitlementsPtr GetEntitlementsInterface() const override;
	UE_API virtual IOnlineStoreV2Ptr GetStoreV2Interface() const override;
	UE_API virtual IOnlinePurchasePtr GetPurchaseInterface() const override;
	UE_API virtual IOnlineEventsPtr GetEventsInterface() const override;
	UE_API virtual IOnlineAchievementsPtr GetAchievementsInterface() const override;
	UE_API virtual IOnlineSharingPtr GetSharingInterface() const override;
	UE_API virtual IOnlineUserPtr GetUserInterface() const override;
	UE_API virtual IOnlineMessagePtr GetMessageInterface() const override;
	UE_API virtual IOnlinePresencePtr GetPresenceInterface() const override;
	UE_API virtual IOnlineChatPtr GetChatInterface() const override;
	UE_API virtual IOnlineStatsPtr GetStatsInterface() const override;
	UE_API virtual IOnlineTurnBasedPtr GetTurnBasedInterface() const override;
	UE_API virtual IOnlineTournamentPtr GetTournamentInterface() const override;
	UE_API virtual bool IsLocalPlayer(const FUniqueNetId& UniqueId) const override;
	UE_API virtual bool Init() override;
	UE_API virtual bool Shutdown() override;
	UE_API virtual bool Exec(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	UE_API virtual bool IsEnabled() const override;
	UE_API virtual FString GetAppId() const override;
	UE_API virtual FText GetOnlineServiceName() const override;

	// FTSTickerObjectBase

	UE_API virtual bool Tick(float DeltaTime) override;

	// FOnlineSubsystemSteam

	/**
	 * Whether or not the Steam Client interfaces are available; these interfaces are only available, if the Steam Client program is running
	 * NOTE: These interfaces are made unavailable, when running a dedicated server
	 *
	 * @return	Whether or not the Steam Client interfaces are available
	 */
	inline bool IsSteamClientAvailable()
	{
		return bSteamworksClientInitialized;
	}

	/**
	 * Whether or not the Steam game server interfaces are available; these interfaces are always available, so long as they were initialized correctly
	 * NOTE: The Steam Client does not need to be running for the game server interfaces to initialize
	 * NOTE: These interfaces are made unavailable, when not running a server
	 *
	 * @return	Whether or not the Steam game server interfaces are available
	 */
	inline bool IsSteamServerAvailable()
	{
		// @todo Steam - add some logic to detect somehow we intended to be a "Steam client" but failed that part
		// yet managed to still initialize the game server aspects of Steam
		return bSteamworksGameServerInitialized;
	}

	/**
	 * @return the steam app id for this app
	 */
	inline uint32 GetSteamAppId() const
	{
		return SteamAppID;
	}

	/**
	 *	@return the port the game has registered for play 
	 */
	inline int32 GetGameServerGamePort() const
	{
		return GameServerGamePort;
	}

	/**
	 *	@return the port the game has registered for incoming server queries
	 */
	inline int32 GetGameServerQueryPort() const
	{
		return GameServerQueryPort;
	}

	/**
	 *	@return if this subsystem is using SteamNetworking functionality 
	 *			or another network layer like SteamSockets
	 */
	inline bool IsUsingSteamNetworking() const
	{
		return bUsingSteamNetworking;
	}

	/**
	 * This delegate fires whenever a steam login has succeeded or failed its async task.
	 * Useful for modules that need to check to see if a user is logged in before running other behavior
	 */
	DEFINE_ONLINE_DELEGATE_ONE_PARAM(OnSteamServerLoginCompleted, bool);
};

namespace FNetworkProtocolTypes
{
	ONLINESUBSYSTEMSTEAM_API extern const FLazyName Steam;
}

typedef TSharedPtr<FOnlineSubsystemSteam, ESPMode::ThreadSafe> FOnlineSubsystemSteamPtr;

#undef UE_API
