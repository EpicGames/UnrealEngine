// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/**
 * Online subsystem module class  (STEAM Implementation)
 * Code related to the loading of the STEAM module
 */
class FOnlineSubsystemSteamModule : public IModuleInterface
{
private:

	/** Class responsible for creating instance(s) of the subsystem */
	class FOnlineFactorySteam* SteamFactory;

public:

	FOnlineSubsystemSteamModule()
        : SteamFactory(NULL)
	{}

	virtual ~FOnlineSubsystemSteamModule() {}

	// IModuleInterface

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

	virtual bool SupportsAutomaticShutdown() override
	{
		return false;
	}
};
